// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AITestSuite : ModuleRules
	{
        public AITestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTasks",
					"AIModule",
				}
				);

			PrecompileForTargets = PrecompileTargetsType.Any;

			PublicDefinitions.Add("WITH_AITESTSUITE 1");
		}
	}
}
