// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"
#include "TestBTDecorator_Blackboard.generated.h"

UCLASS(meta = (HiddenNode))
class UTestBTDecorator_Blackboard : public UBTDecorator_Blackboard
{
	GENERATED_UCLASS_BODY()

public:
	
	UPROPERTY()
	int32 LogIndexBecomeRelevant;

	UPROPERTY()
	int32 LogIndexCeaseRelevant;

	UPROPERTY()
	int32 LogIndexCalculate;

protected:
	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;

private:
	void LogExecution(int32 LogNumber) const;
};
