// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "BehaviorTree/Decorators/BTDecorator_BlueprintBase.h"

#include "TestBTDecorator_Blueprint.generated.h"

UENUM()
enum class EBPConditionType
{
	NoCondition,
	TrueCondition,
	FalseCondition
};

UCLASS(meta = (HiddenNode))
class UTestBTDecorator_Blueprint : public UBTDecorator_BlueprintBase
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	EBPConditionType BPConditionType = EBPConditionType::TrueCondition;
	
	UPROPERTY()
	int32 LogIndexBecomeRelevant;

	UPROPERTY()
	int32 LogIndexCeaseRelevant;

	UPROPERTY()
	int32 LogIndexCalculate;

	UPROPERTY()
	FName ObservingKeyName = NAME_None;

protected:
	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;

private:
	void LogExecution(int32 LogNumber) const;
};
