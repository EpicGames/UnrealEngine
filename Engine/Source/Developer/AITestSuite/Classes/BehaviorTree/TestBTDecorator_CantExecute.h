// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "TestBTDecorator_CantExecute.generated.h"

UCLASS(meta=(HiddenNode))
class UTestBTDecorator_CantExecute : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
};
