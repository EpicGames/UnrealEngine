// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "TestBTDecorator_DelayedAbort.generated.h"

struct FBTDelayedAbortMemory
{
	uint64 EndFrameIdx;
};

UCLASS(meta = (HiddenNode))
class UTestBTDecorator_DelayedAbort : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	int32 DelayTicks;

	UPROPERTY()
	bool bOnlyOnce;

	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	virtual uint16 GetInstanceMemorySize() const override;
	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
};
