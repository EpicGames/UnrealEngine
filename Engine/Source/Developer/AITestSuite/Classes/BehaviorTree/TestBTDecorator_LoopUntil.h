// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/Decorators/BTDecorator_LoopUntil.h"
#include "TestBTDecorator_LoopUntil.generated.h"

/**
 * Loop until test version used for unit tests
 */
UCLASS(meta=(HiddenNode), MinimalAPI)
class UTestBTDecorator_LoopUntil : public UBTDecorator_LoopUntil
{
	GENERATED_BODY()
public:

	UTestBTDecorator_LoopUntil(const FObjectInitializer& ObjectInitializer);
	void SetRequiredResult(const EBTNodeResult::Type InRequiredResult);
};
