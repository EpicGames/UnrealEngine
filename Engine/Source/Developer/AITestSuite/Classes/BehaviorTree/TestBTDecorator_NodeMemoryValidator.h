// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "BehaviorTree/BTDecorator.h"
#include "TestBTDecorator_NodeMemoryValidator.generated.h"

struct FBTNodeMemoryValidatorDecoratorMemory
{
	static constexpr uint32 TestValue = 0xABBAABBA;
	uint32 Dummy;
};

UCLASS(meta=(HiddenNode))
class UTestBTDecorator_NodeMemoryValidator : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

protected:
	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	virtual void OnNodeActivation(FBehaviorTreeSearchData& SearchData) override;
	virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;
	virtual void OnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) override;
	virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;

	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	virtual uint16 GetInstanceMemorySize() const override;

private:

	template<typename T>
	T* GetCheckedNodeMemory(FBehaviorTreeSearchData& SearchData) const
	{
		checkf(SearchData.OwnerComp.FindInstanceContainingNode(this) == SearchData.OwnerComp.GetActiveInstanceIdx(), TEXT("Accessing memory from the wrong instance"));
		return GetNodeMemory<T>(SearchData);
	}

	template<typename T>
	const T* GetCheckedNodeMemory(const FBehaviorTreeSearchData& SearchData) const
	{
		checkf(SearchData.OwnerComp.FindInstanceContainingNode(this) == SearchData.OwnerComp.GetActiveInstanceIdx(), TEXT("Accessing memory from the wrong instance"));
		return GetNodeMemory<T>(SearchData);
	}
};
