// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTService.h"
#include "TestBTStopAction.h"
#include "TestBTService_BTStopAction.generated.h"

UENUM()
enum class EBTTestServiceStopTiming : uint8
{
	DuringBecomeRelevant,
	DuringTick,
	DuringCeaseRelevant,
};

UCLASS(meta = (HiddenNode))
class UTestBTService_BTStopAction : public UBTService
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	int32 LogIndex;

	UPROPERTY()
	EBTTestServiceStopTiming StopTiming;

	UPROPERTY()
	EBTTestStopAction StopAction;

	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
};
