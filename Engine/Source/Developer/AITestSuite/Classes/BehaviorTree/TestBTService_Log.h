// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTService.h"
#include "TestBTService_Log.generated.h"

UCLASS(meta = (HiddenNode))
class UTestBTService_Log : public UBTService
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	int32 LogActivation;

	UPROPERTY()
	int32 LogDeactivation;

	UPROPERTY()
	FName KeyNameTick;

	UPROPERTY()
	FName KeyNameBecomeRelevant;

	UPROPERTY()
	FName KeyNameCeaseRelevant;

	UPROPERTY()
	int32 LogTick;

	UPROPERTY()
	int32 TicksDelaySetKeyNameTick;

	UPROPERTY()
	int32 NumTicks;

	UPROPERTY()
	bool bToggleValue;

	virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

	void SetFlagOnTick(FName InKeyNameTick, bool bInCallTickOnSearchStart = false);
};
