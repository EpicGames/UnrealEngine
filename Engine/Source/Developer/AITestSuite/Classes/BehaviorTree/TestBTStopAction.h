// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TestBTStopAction.generated.h"

class UBehaviorTreeComponent;

UENUM()
enum class EBTTestStopAction : uint8
{
	StopTree,
	UnInitialize,
	Cleanup,
	Restart_ForceReevaluateRootNode,
	Restart_Complete,
	StartTree,
};

void DoBTStopAction(UBehaviorTreeComponent& OwnerComp, const EBTTestStopAction StopAction);