// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "TestBTStopAction.h"
#include "TestBTTask_BTStopAction.generated.h"

UENUM()
enum class EBTTestTaskStopTiming : uint8
{
	DuringExecute,
	DuringTick,
	DuringAbort,
	DuringFinish,
};

UCLASS(meta=(HiddenNode))
class UTestBTTask_BTStopAction : public UBTTaskNode
{
	GENERATED_BODY()

public:
	UTestBTTask_BTStopAction(const FObjectInitializer& ObjectInitializer);

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	virtual void TickTask(class UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;

	UPROPERTY()
	EBTTestTaskStopTiming StopTiming;

	UPROPERTY()
	EBTTestStopAction StopAction;

	UPROPERTY()
	int32 LogIndex;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> LogResult;
};
