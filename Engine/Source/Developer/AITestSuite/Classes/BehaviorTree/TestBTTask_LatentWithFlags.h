// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_LatentWithFlags.generated.h"

struct FBTLatentTaskMemory
{
	uint64 FlagFrameIdx;
	uint64 EndFrameIdx;
	uint8 bFlagSet : 1;
	uint8 bIsAborting : 1;
};

UENUM()
enum class EBTTestChangeFlagBehavior : uint8
{
	Set,
	Toggle
};

UCLASS(meta = (HiddenNode))
class UTestBTTask_LatentWithFlags : public UBTTaskNode
{
	GENERATED_BODY()

public:

	UTestBTTask_LatentWithFlags();

	UPROPERTY()
	int32 LogIndexExecuteStart = 0;

	UPROPERTY()
	int32 LogIndexExecuting = -1;

	UPROPERTY()
	int32 LogIndexExecuteFinish = 0;

	UPROPERTY()
	int32 LogIndexAbortStart = 0;

	UPROPERTY()
	int32 LogIndexAborting = -1;

	UPROPERTY()
	int32 LogIndexAbortFinish = 0;

	/** Num of ticks before 'execute start' and `set execute flag` and then the same num of ticks before `execute finish` */
	UPROPERTY()
	int32 ExecuteHalfTicks = 2;

	/** Num of ticks before 'abort start' and `set abort flag` and then the same num of ticks before `abort finish` */
	UPROPERTY()
	int32 AbortHalfTicks = 2;

	UPROPERTY()
	FName KeyNameExecute = TEXT("Bool1");

	UPROPERTY()
	FName KeyNameAbort = TEXT("Bool2");

	UPROPERTY()
	EBTTestChangeFlagBehavior ChangeFlagBehavior = EBTTestChangeFlagBehavior::Set;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> LogResult = EBTNodeResult::Succeeded;

protected:
	virtual EBTNodeResult::Type ExecuteTask(class UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual uint16 GetInstanceMemorySize() const override;
	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	void LogExecution(class UBehaviorTreeComponent& OwnerComp, int32 LogNumber);

	virtual void TickTask(class UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

	void ChangeFlag(UBehaviorTreeComponent& OwnerComp, FName FlagToChange) const;
};