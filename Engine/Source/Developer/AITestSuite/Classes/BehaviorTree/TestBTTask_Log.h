// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_Log.generated.h"

struct FBTLogTaskMemory
{
	uint64 EndFrameIdx;
};

UCLASS(meta=(HiddenNode))
class UTestBTTask_Log : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	int32 LogIndex;

	UPROPERTY()
	int32 LogFinished;

	UPROPERTY()
	int32 ExecutionTicks;

	UPROPERTY()
	int32 LogTickIndex;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> LogResult;

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual uint16 GetInstanceMemorySize() const override;
	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

	void LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber);

protected:
	virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
};
