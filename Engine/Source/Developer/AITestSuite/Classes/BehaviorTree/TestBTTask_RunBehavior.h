// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "TestBTTask_RunBehavior.generated.h"

UCLASS(MinimalAPI, meta = (HiddenNode))
class UTestBTTask_RunBehavior : public UBTTask_RunBehavior
{
	GENERATED_UCLASS_BODY()
	
	/**
	 * Sets the subtree asset to run.
	 * @note This method has been added for test purpose only
	 * and should be called only before the task gets activated.
	 */
	void SetSubtreeAsset(UBehaviorTree* Asset)
	{
		BehaviorAsset = Asset;
	}
};
