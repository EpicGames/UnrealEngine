// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_SetFlag.generated.h"

UCLASS(meta=(HiddenNode))
class UTestBTTask_SetFlag : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FName KeyName;

	UPROPERTY()
	bool bValue;

	UPROPERTY()
	FName OnAbortKeyName;

	UPROPERTY()
	bool bOnAbortValue;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> TaskResult;

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
};
