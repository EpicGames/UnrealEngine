// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_SetValuesWithLogs.generated.h"

struct FBTSetValueTaskMemory
{
	uint64 EndFrameIdx;
	uint64 EndFrameIdx2;
};

UCLASS(meta=(HiddenNode))
class UTestBTTask_SetValuesWithLogs : public UBTTaskNode
{
	GENERATED_BODY()

public:
	UTestBTTask_SetValuesWithLogs(const FObjectInitializer& ObjectInitializer);

	/** Index logged when we execute the task */
	UPROPERTY()
	int32 LogIndex;

	/** Index logged when we finish the task */
	UPROPERTY()
	int32 LogFinished;

	/** Number of execution ticks until we set Blackboard KeyName2 Value2 */
	UPROPERTY()
	int32 ExecutionTicks1;

	/** Number of execution ticks after ExecutionTicks1 that we end the task */
	UPROPERTY()
	int32 ExecutionTicks2;

	/** Index logged when we tick the task */
	UPROPERTY()
	int32 LogTickIndex;

	/** Blackboard keyname used when we start the task */
	UPROPERTY()
	FName KeyName1;

	/** Blackboard value set when we start the task */
	UPROPERTY()
	int32 Value1;

	/** Blackboard keyname used after ExecutionTicks1 execution ticks */
	UPROPERTY()
	FName KeyName2;

	/** Blackboard value set after ExecutionTicks1 execution ticks */
	UPROPERTY()
	int32 Value2;

	/** Blackboard keyname used when we abort the task */
	UPROPERTY()
	FName OnAbortKeyName;

	/** Blackboard value set when we abort the task */
	UPROPERTY()
	int32 OnAbortValue;

	/** Result when we finish the task */
	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> TaskResult;

protected:
	virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	void LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber);
	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual uint16 GetInstanceMemorySize() const override;
	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
};
