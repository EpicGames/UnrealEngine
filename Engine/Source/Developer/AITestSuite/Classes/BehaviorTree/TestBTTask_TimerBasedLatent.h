// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_TimerBasedLatent.generated.h"

struct FBTTimerBasedLatentTaskMemory
{
	FTimerHandle TimerHandle;
	uint8 bIsAborting : 1;
};

UCLASS(meta = (HiddenNode))
class UTestBTTask_TimerBasedLatent : public UBTTaskNode
{
	GENERATED_BODY()

public:

	UTestBTTask_TimerBasedLatent();

	UPROPERTY()
	int32 LogIndexExecuteStart = INDEX_NONE;

	UPROPERTY()
	int32 LogIndexExecuteFinish = INDEX_NONE;

	UPROPERTY()
	int32 LogIndexAbortStart = INDEX_NONE;

	UPROPERTY()
	int32 LogIndexAbortFinish = INDEX_NONE;

	/** Num of ticks from 'execute start' to 'execute finish' */
	UPROPERTY()
	int32 NumTicksExecuting = 0;

	/** Num of ticks from 'abort start' to 'abort finish' */
	UPROPERTY()
	int32 NumTicksAborting = 0;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> LogResult = EBTNodeResult::Succeeded;

protected:
	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual uint16 GetInstanceMemorySize() const override;
	virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	void LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber) const;
};