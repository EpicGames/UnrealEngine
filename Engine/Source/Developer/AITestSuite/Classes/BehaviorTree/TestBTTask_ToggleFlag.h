// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "TestBTTask_ToggleFlag.generated.h"

UCLASS(meta=(HiddenNode))
class UTestBTTask_ToggleFlag : public UBTTaskNode
{
	GENERATED_BODY()

public:
	UTestBTTask_ToggleFlag(const FObjectInitializer& ObjectInitializer);

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	UPROPERTY()
	int32 LogIndex;

	UPROPERTY()
	FName KeyName;

	UPROPERTY()
	int32 NumToggles;

	UPROPERTY()
	TEnumAsByte<EBTNodeResult::Type> TaskResult;
};
