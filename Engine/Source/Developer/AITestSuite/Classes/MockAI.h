// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/Object.h"
#include "AITestsCommon.h"
#include "Tickable.h"

#include "MockAI.generated.h"

class UAIPerceptionComponent;
class UBlackboardComponent;
class UBrainComponent;
class UMockAI;

struct FTestTickHelper : FTickableGameObject
{
	TWeakObjectPtr<class UMockAI> Owner;

	FTestTickHelper() : Owner(nullptr) {}
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return Owner.IsValid(); }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual TStatId GetStatId() const override;
};

UCLASS()
class UMockAI : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual ~UMockAI() override;

	FTestTickHelper TickHelper;

	UPROPERTY()
	TObjectPtr<AActor> Actor = nullptr;

	UPROPERTY()
	TObjectPtr<UBlackboardComponent> BBComp = nullptr;

	UPROPERTY()
	TObjectPtr<UBrainComponent> BrainComp = nullptr;

	UPROPERTY()
	TObjectPtr<UAIPerceptionComponent> PerceptionComp = nullptr;

	template<typename TBrainClass>
	void UseBrainComponent()
	{
		BrainComp = NewObject<TBrainClass>(Actor);
	}

	void UseBlackboardComponent();
	void UsePerceptionComponent();

	void SetEnableTicking(bool bShouldTick);

	virtual void TickMe(float DeltaTime);
};
