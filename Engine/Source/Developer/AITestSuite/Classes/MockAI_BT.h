// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "MockAI.h"
#include "MockAI_BT.generated.h"

class UBehaviorTree;
class UBehaviorTreeComponent;

UCLASS()
class UMockAI_BT : public UMockAI
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UBehaviorTreeComponent> BTComp;

	static TArray<int32> ExecutionLog;
	TArray<int32> ExpectedResult;

	bool IsRunning() const;
	void RunBT(UBehaviorTree& BTAsset, EBTExecutionMode::Type RunType = EBTExecutionMode::SingleRun);
};
