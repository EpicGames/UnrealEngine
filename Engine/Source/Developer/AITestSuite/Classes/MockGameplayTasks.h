// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameplayTasksComponent.h"
#include "MockGameplayTasks.generated.h"

class AActor;
template<typename ValueType> struct FTestLogger;

namespace ETestTaskMessage
{
	enum Type
	{
		Activate,
		Tick,
		ExternalConfirm,
		ExternalCancel,
		Ended
	};
}

UCLASS()
class UMockTask_Log : public UGameplayTask
{
	GENERATED_BODY()

protected:
	FTestLogger<int32>* Logger;
	bool bShoudEndAsPartOfActivation;

public:
	UMockTask_Log(const FObjectInitializer& ObjectInitializer);

	static UMockTask_Log* CreateTask(IGameplayTaskOwnerInterface& TaskOwner, FTestLogger<int32>& InLogger, const FGameplayResourceSet& Resources = FGameplayResourceSet(), uint8 Priority = FGameplayTasks::DefaultPriority);

protected:
	virtual void Activate() override;
	virtual void OnDestroy(bool bOwnerFinished) override;

public:
	virtual void TickTask(float DeltaTime) override;
	virtual void ExternalConfirm(bool bEndTask) override;
	virtual void ExternalCancel() override;

	// testing only hack-functions
	void EnableTick() { bTickingTask = true; }

	void SetInstaEnd(bool bNewValue) { bShoudEndAsPartOfActivation = bNewValue; }
};

//
// a Testing-time component that is a way to access UGameplayTasksComponent's protected properties
//
UCLASS()
class UMockGameplayTasksComponent : public UGameplayTasksComponent
{
	GENERATED_BODY()

public:	
	int32 GetTaskPriorityQueueSize() const { return TaskPriorityQueue.Num(); }
};

UCLASS()
class UMockGameplayTaskOwner : public UObject, public IGameplayTaskOwnerInterface
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UGameplayTasksComponent> GTComponent;

	virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override { return GTComponent; }	
	virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const { return nullptr; }
};
