// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_Blackboard.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_Blackboard)

UTestBTDecorator_Blackboard::UTestBTDecorator_Blackboard(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
, LogIndexBecomeRelevant(-1)
, LogIndexCeaseRelevant(-1)
, LogIndexCalculate(-1)
{
}

void UTestBTDecorator_Blackboard::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	LogExecution(LogIndexBecomeRelevant);
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);
}

void UTestBTDecorator_Blackboard::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	LogExecution(LogIndexCeaseRelevant);
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);
}

bool UTestBTDecorator_Blackboard::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	LogExecution(LogIndexCalculate);
	return Super::CalculateRawConditionValue(OwnerComp, NodeMemory);
}

void UTestBTDecorator_Blackboard::LogExecution(int32 LogNumber) const
{
	if (LogNumber >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}

