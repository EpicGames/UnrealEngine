// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_Blueprint.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_Blueprint)

UTestBTDecorator_Blueprint::UTestBTDecorator_Blueprint(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
, LogIndexBecomeRelevant(INDEX_NONE)
, LogIndexCeaseRelevant(INDEX_NONE)
, LogIndexCalculate(INDEX_NONE)
{
}

void UTestBTDecorator_Blueprint::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	LogExecution(LogIndexBecomeRelevant);
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);
}

void UTestBTDecorator_Blueprint::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	LogExecution(LogIndexCeaseRelevant);
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);
}

bool UTestBTDecorator_Blueprint::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	LogExecution(LogIndexCalculate);
	return BPConditionType == EBPConditionType::TrueCondition ? true : false;
}

void UTestBTDecorator_Blueprint::PostLoad()
{
	// Skip UBTDecorator_BlueprintBase as we are faking the fetching of blueprint BB keys
	UBTDecorator::PostLoad();

	// Let's fake this blueprint has a condition check as we overriden the CalculateRawConditionValue
	PerformConditionCheckImplementations = BPConditionType == EBPConditionType::NoCondition ? 0 : 1;

	if (!ObservingKeyName.IsNone())
	{
		ObservedKeyNames.Reset();
		ObservedKeyNames.Add(ObservingKeyName);
		bIsObservingBB = true;
	}

	if (PerformConditionCheckImplementations || bIsObservingBB)
	{
		bNotifyBecomeRelevant = true;
		bNotifyCeaseRelevant = true;
	}
}

void UTestBTDecorator_Blueprint::PostInitProperties()
{
	// Skip UBTDecorator_BlueprintBase as we are faking the fetching of blueprint BB keys
	UBTDecorator::PostInitProperties();
}

void UTestBTDecorator_Blueprint::LogExecution(int32 LogNumber) const
{
	if (LogNumber != INDEX_NONE)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}

