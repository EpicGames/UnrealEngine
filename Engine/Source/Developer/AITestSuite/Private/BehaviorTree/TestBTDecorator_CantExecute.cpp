// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_CantExecute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_CantExecute)

UTestBTDecorator_CantExecute::UTestBTDecorator_CantExecute(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = TEXT("Can't Exexcute");

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

bool UTestBTDecorator_CantExecute::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	return false;
}

