// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_DelayedAbort.h"
#include "AITestsCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_DelayedAbort)

UTestBTDecorator_DelayedAbort::UTestBTDecorator_DelayedAbort(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Delayed Abort";
	DelayTicks = 5;
	bOnlyOnce = true;

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = true;
	FlowAbortMode = EBTFlowAbortMode::Self;
}

void UTestBTDecorator_DelayedAbort::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTDelayedAbortMemory* MyMemory = (FBTDelayedAbortMemory*)NodeMemory;
	MyMemory->EndFrameIdx = FAITestHelpers::FramesCounter() + DelayTicks;
}

void UTestBTDecorator_DelayedAbort::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTDelayedAbortMemory* MyMemory = (FBTDelayedAbortMemory*)NodeMemory;

	if (FAITestHelpers::FramesCounter() >= MyMemory->EndFrameIdx)
	{
		OwnerComp.RequestExecution(this);
		MyMemory->EndFrameIdx = bOnlyOnce ? MAX_uint64 : 0;
	}
}

uint16 UTestBTDecorator_DelayedAbort::GetInstanceMemorySize() const
{
	return sizeof(FBTDelayedAbortMemory);
}

void UTestBTDecorator_DelayedAbort::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTDelayedAbortMemory>(NodeMemory, InitType);
}

void UTestBTDecorator_DelayedAbort::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTDelayedAbortMemory>(NodeMemory, CleanupType);
}