// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_LoopUntil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_LoopUntil)

UTestBTDecorator_LoopUntil::UTestBTDecorator_LoopUntil(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	NodeName = TEXT("Test Loop Until");

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

void UTestBTDecorator_LoopUntil::SetRequiredResult(const EBTNodeResult::Type InRequiredResult)
{
	RequiredResult = InRequiredResult;
}
