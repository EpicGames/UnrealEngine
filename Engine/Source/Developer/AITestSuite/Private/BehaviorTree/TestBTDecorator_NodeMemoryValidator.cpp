// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTDecorator_NodeMemoryValidator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTDecorator_NodeMemoryValidator)

UTestBTDecorator_NodeMemoryValidator::UTestBTDecorator_NodeMemoryValidator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = TEXT("NodeMemoryValidator");

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
}

void UTestBTDecorator_NodeMemoryValidator::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

void UTestBTDecorator_NodeMemoryValidator::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

void UTestBTDecorator_NodeMemoryValidator::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

void UTestBTDecorator_NodeMemoryValidator::OnNodeActivation(FBehaviorTreeSearchData& SearchData)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = GetCheckedNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(SearchData);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

void UTestBTDecorator_NodeMemoryValidator::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = GetCheckedNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(SearchData);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

void UTestBTDecorator_NodeMemoryValidator::OnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult)
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = GetCheckedNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(SearchData);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
}

bool UTestBTDecorator_NodeMemoryValidator::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
	return Super::CalculateRawConditionValue(OwnerComp, NodeMemory);
}

uint16 UTestBTDecorator_NodeMemoryValidator::GetInstanceMemorySize() const
{
	return sizeof(FBTNodeMemoryValidatorDecoratorMemory);
}

void UTestBTDecorator_NodeMemoryValidator::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = InitializeNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory, InitType);
	if (InitType == EBTMemoryInit::Initialize)
	{
		DecoratorMemory->Dummy = FBTNodeMemoryValidatorDecoratorMemory::TestValue;
	}
}

void UTestBTDecorator_NodeMemoryValidator::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	const FBTNodeMemoryValidatorDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory);
	check(DecoratorMemory->Dummy == FBTNodeMemoryValidatorDecoratorMemory::TestValue);
	CleanupNodeMemory<FBTNodeMemoryValidatorDecoratorMemory>(NodeMemory, CleanupType);
}
