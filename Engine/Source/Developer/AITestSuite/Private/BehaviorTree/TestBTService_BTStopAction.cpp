// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTService_BTStopAction.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTService_BTStopAction)

UTestBTService_BTStopAction::UTestBTService_BTStopAction(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "StopTreeService";

	INIT_SERVICE_NODE_NOTIFY_FLAGS();

	LogIndex = INDEX_NONE;
	StopTiming = EBTTestServiceStopTiming::DuringTick;

	// Force the service to tick every frame
	Interval = 0.0f;
	RandomDeviation = 0.0f;
}

void UTestBTService_BTStopAction::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);

	if (StopTiming == EBTTestServiceStopTiming::DuringBecomeRelevant)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		DoBTStopAction(OwnerComp, StopAction);
	}
}

void UTestBTService_BTStopAction::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);

	if (StopTiming == EBTTestServiceStopTiming::DuringCeaseRelevant)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		OwnerComp.StopTree();
		DoBTStopAction(OwnerComp, StopAction);
	}
}

void UTestBTService_BTStopAction::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	Super::TickNode(OwnerComp, NodeMemory, DeltaSeconds);

	if (StopTiming == EBTTestServiceStopTiming::DuringTick)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		OwnerComp.StopTree();
		DoBTStopAction(OwnerComp, StopAction);
	}
}