// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTService_Log.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTService_Log)

UTestBTService_Log::UTestBTService_Log(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "LogService";

	INIT_SERVICE_NODE_NOTIFY_FLAGS();

	LogActivation = INDEX_NONE;
	LogDeactivation = INDEX_NONE;
	KeyNameTick = NAME_None;
	LogTick = INDEX_NONE;
	KeyNameBecomeRelevant = NAME_None;
	KeyNameCeaseRelevant = NAME_None;
	bToggleValue = false;

	// Force the service to tick every frame
	Interval = 0.0f;
	RandomDeviation = 0.0f;
}

void UTestBTService_Log::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);

	if (KeyNameBecomeRelevant != NAME_None)
	{
		OwnerComp.GetBlackboardComponent()->SetValueAsBool(KeyNameBecomeRelevant, bToggleValue ? !OwnerComp.GetBlackboardComponent()->GetValueAsBool(KeyNameBecomeRelevant) : true);
	}

	if (LogActivation >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogActivation);
	}
}

void UTestBTService_Log::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);

	if (KeyNameCeaseRelevant != NAME_None)
	{
		OwnerComp.GetBlackboardComponent()->SetValueAsBool(KeyNameCeaseRelevant, bToggleValue ? !OwnerComp.GetBlackboardComponent()->GetValueAsBool(KeyNameCeaseRelevant) : true);
	}

	if (LogDeactivation >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogDeactivation);
	}
}

void UTestBTService_Log::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	Super::TickNode(OwnerComp, NodeMemory, DeltaSeconds);

	if (KeyNameTick != NAME_None && NumTicks >= TicksDelaySetKeyNameTick)
	{
		OwnerComp.GetBlackboardComponent()->SetValueAsBool(KeyNameTick, bToggleValue ? !OwnerComp.GetBlackboardComponent()->GetValueAsBool(KeyNameTick) : true);
	}

	if (LogTick >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogTick);
	}
	++NumTicks;
}

void UTestBTService_Log::SetFlagOnTick(FName InKeyNameTick, bool bInCallTickOnSearchStart /* = false */)
{
	KeyNameTick = InKeyNameTick;
	bCallTickOnSearchStart = bInCallTickOnSearchStart;
}

