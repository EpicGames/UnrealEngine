// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_BTStopAction.h"
#include "BehaviorTree/BehaviorTree.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_BTStopAction)

UTestBTTask_BTStopAction::UTestBTTask_BTStopAction(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	StopTiming = EBTTestTaskStopTiming::DuringExecute;
	LogResult = EBTNodeResult::Succeeded;
	LogIndex = INDEX_NONE;
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UTestBTTask_BTStopAction::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (StopTiming == EBTTestTaskStopTiming::DuringExecute)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		DoBTStopAction(OwnerComp, StopAction);
		return LogResult;
	}

	return EBTNodeResult::InProgress;
}

EBTNodeResult::Type UTestBTTask_BTStopAction::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (StopTiming == EBTTestTaskStopTiming::DuringAbort)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		DoBTStopAction(OwnerComp, StopAction);
	}
	return EBTNodeResult::Aborted;
}

void UTestBTTask_BTStopAction::TickTask(class UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	if (StopTiming == EBTTestTaskStopTiming::DuringTick)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		DoBTStopAction(OwnerComp, StopAction);
		FinishLatentTask(OwnerComp, LogResult);
	}
}

void UTestBTTask_BTStopAction::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	if (StopTiming == EBTTestTaskStopTiming::DuringFinish)
	{
		UMockAI_BT::ExecutionLog.Add(LogIndex);
		DoBTStopAction(OwnerComp, StopAction);
	}
}


void DoBTStopAction(UBehaviorTreeComponent& OwnerComp, const EBTTestStopAction StopAction)
{
	switch (StopAction)
	{
	case EBTTestStopAction::StopTree:
		OwnerComp.StopTree();
		break;
	case EBTTestStopAction::UnInitialize:
		OwnerComp.UninitializeComponent();
		break;
	case EBTTestStopAction::Cleanup:
		OwnerComp.Cleanup();
		break;
	case EBTTestStopAction::Restart_ForceReevaluateRootNode:
		OwnerComp.RestartTree(EBTRestartMode::ForceReevaluateRootNode);
		break;
	case EBTTestStopAction::Restart_Complete:
		OwnerComp.RestartTree(EBTRestartMode::CompleteRestart);
		break;
	case EBTTestStopAction::StartTree:
		UBehaviorTree* NewBTAsset = NewObject<UBehaviorTree>(GetTransientPackage(), NAME_None, RF_NoFlags, OwnerComp.GetRootTree());
		OwnerComp.StartTree(*NewBTAsset);
		break;
	}
}