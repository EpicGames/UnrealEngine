// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_LatentWithFlags.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_LatentWithFlags)

UTestBTTask_LatentWithFlags::UTestBTTask_LatentWithFlags()
{
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UTestBTTask_LatentWithFlags::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTLatentTaskMemory* MyMemory = CastInstanceNodeMemory<FBTLatentTaskMemory>(NodeMemory);
	MyMemory->FlagFrameIdx = ExecuteHalfTicks + FAITestHelpers::FramesCounter();
	MyMemory->EndFrameIdx = MyMemory->FlagFrameIdx + ExecuteHalfTicks;
	MyMemory->bFlagSet = false;
	MyMemory->bIsAborting = false;

	LogExecution(OwnerComp, LogIndexExecuteStart);
	if (ExecuteHalfTicks == 0)
	{
		ChangeFlag(OwnerComp, KeyNameExecute);
		MyMemory->bFlagSet = true;

		LogExecution(OwnerComp, LogIndexExecuteFinish);
		return LogResult;
	}

	return EBTNodeResult::InProgress;
}

EBTNodeResult::Type UTestBTTask_LatentWithFlags::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTLatentTaskMemory* MyMemory = CastInstanceNodeMemory<FBTLatentTaskMemory>(NodeMemory);
	MyMemory->FlagFrameIdx = AbortHalfTicks + FAITestHelpers::FramesCounter();
	MyMemory->EndFrameIdx = MyMemory->FlagFrameIdx + AbortHalfTicks;
	MyMemory->bFlagSet = false;
	MyMemory->bIsAborting = true;

	LogExecution(OwnerComp, LogIndexAbortStart);
	if (AbortHalfTicks == 0)
	{
		ChangeFlag(OwnerComp, KeyNameAbort);
		MyMemory->bFlagSet = true;

		LogExecution(OwnerComp, LogIndexAbortFinish);
		return EBTNodeResult::Aborted;
	}

	return EBTNodeResult::InProgress;
}

void UTestBTTask_LatentWithFlags::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTLatentTaskMemory* MyMemory = CastInstanceNodeMemory<FBTLatentTaskMemory>(NodeMemory);

	LogExecution(OwnerComp, MyMemory->bIsAborting ? LogIndexAborting : LogIndexExecuting);

	if (!MyMemory->bFlagSet && FAITestHelpers::FramesCounter() >= MyMemory->FlagFrameIdx)
	{
		MyMemory->bFlagSet = true;
		ChangeFlag(OwnerComp, MyMemory->bIsAborting ? KeyNameAbort : KeyNameExecute);
	}

	if (FAITestHelpers::FramesCounter() >= MyMemory->EndFrameIdx)
	{
		if (MyMemory->bIsAborting)
		{
			LogExecution(OwnerComp, LogIndexAbortFinish);
			FinishLatentAbort(OwnerComp);
		}
		else
		{
			LogExecution(OwnerComp, LogIndexExecuteFinish);
			FinishLatentTask(OwnerComp, LogResult);
		}
	}
}

void UTestBTTask_LatentWithFlags::ChangeFlag(UBehaviorTreeComponent& OwnerComp, FName FlagToChange) const
{
	switch(ChangeFlagBehavior)
	{
		case EBTTestChangeFlagBehavior::Set:
			OwnerComp.GetBlackboardComponent()->SetValueAsBool(FlagToChange, true);
			break;
		case EBTTestChangeFlagBehavior::Toggle:
			OwnerComp.GetBlackboardComponent()->SetValueAsBool(FlagToChange, !OwnerComp.GetBlackboardComponent()->GetValueAsBool(FlagToChange));
			break;
	}
}

uint16 UTestBTTask_LatentWithFlags::GetInstanceMemorySize() const
{
	return sizeof(FBTLatentTaskMemory);
}

void UTestBTTask_LatentWithFlags::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTLatentTaskMemory>(NodeMemory, InitType);
}

void UTestBTTask_LatentWithFlags::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTLatentTaskMemory>(NodeMemory, CleanupType);
}

void UTestBTTask_LatentWithFlags::LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber)
{
	if (LogNumber >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}
