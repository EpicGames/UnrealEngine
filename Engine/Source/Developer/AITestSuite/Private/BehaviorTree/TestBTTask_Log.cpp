// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_Log.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_Log)

UTestBTTask_Log::UTestBTTask_Log(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Log";
	ExecutionTicks = 0;
	LogIndex = 0;
	LogFinished = -1;
	LogResult = EBTNodeResult::Succeeded;
	LogTickIndex = -1;

	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UTestBTTask_Log::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTLogTaskMemory* MyMemory = CastInstanceNodeMemory<FBTLogTaskMemory>(NodeMemory);
	MyMemory->EndFrameIdx = ExecutionTicks + FAITestHelpers::FramesCounter();

	LogExecution(OwnerComp, LogIndex);
	if (ExecutionTicks == 0)
	{
		return LogResult;
	}

	return EBTNodeResult::InProgress;
}

void UTestBTTask_Log::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTLogTaskMemory* MyMemory = CastInstanceNodeMemory<FBTLogTaskMemory>(NodeMemory);

	if (LogTickIndex != -1)
	{
		LogExecution(OwnerComp, LogTickIndex);
	}

	if (FAITestHelpers::FramesCounter() >= MyMemory->EndFrameIdx)
	{
		LogExecution(OwnerComp, LogFinished);
		FinishLatentTask(OwnerComp, LogResult);
	}
}

uint16 UTestBTTask_Log::GetInstanceMemorySize() const
{
	return sizeof(FBTLogTaskMemory);
}

void UTestBTTask_Log::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTLogTaskMemory>(NodeMemory, InitType);
}

void UTestBTTask_Log::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTLogTaskMemory>(NodeMemory, CleanupType);
}

void UTestBTTask_Log::LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber)
{
	if (LogNumber >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}

