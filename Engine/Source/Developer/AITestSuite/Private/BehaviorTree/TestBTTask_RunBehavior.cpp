// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_RunBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_RunBehavior)

UTestBTTask_RunBehavior::UTestBTTask_RunBehavior(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = TEXT("Test Run Behavior");
}
