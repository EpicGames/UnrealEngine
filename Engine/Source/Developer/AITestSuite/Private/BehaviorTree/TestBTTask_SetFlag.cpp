// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_SetFlag.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_SetFlag)

UTestBTTask_SetFlag::UTestBTTask_SetFlag(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Log";
	TaskResult = EBTNodeResult::Succeeded;
	KeyName = TEXT("Bool1");
	bValue = true;
	OnAbortKeyName = FName();
	bOnAbortValue = true;
}

EBTNodeResult::Type UTestBTTask_SetFlag::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Bool>(KeyName, bValue);
	return TaskResult;
}

EBTNodeResult::Type UTestBTTask_SetFlag::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (OnAbortKeyName.IsValid())
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Bool>(OnAbortKeyName, bOnAbortValue);
	}
	return EBTNodeResult::Aborted;
}
