// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_SetValue.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_SetValue)

UTestBTTask_SetValue::UTestBTTask_SetValue(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "SetValue";
	TaskResult = EBTNodeResult::Succeeded;
	KeyName = TEXT("Int");
	Value = 1;
	OnAbortKeyName = FName();
	OnAbortValue = 1;
}

EBTNodeResult::Type UTestBTTask_SetValue::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(KeyName, Value);
	return TaskResult;
}

EBTNodeResult::Type UTestBTTask_SetValue::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (OnAbortKeyName.IsValid())
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(OnAbortKeyName, OnAbortValue);
	}
	return EBTNodeResult::Aborted;	
}

