// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_SetValuesWithLogs.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AITestsCommon.h"
#include "MockAI_BT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_SetValuesWithLogs)

UTestBTTask_SetValuesWithLogs::UTestBTTask_SetValuesWithLogs(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "SetValues";
	ExecutionTicks1 = 0;
	ExecutionTicks2 = 0;
	LogIndex = 0;
	LogFinished = -1;
	LogTickIndex = -1;
	TaskResult = EBTNodeResult::Succeeded;
	KeyName1 = TEXT("Int");
	Value1 = 1;
	KeyName2 = TEXT("Int2");
	Value2 = 1;
	OnAbortKeyName = FName();
	OnAbortValue = 1;

	bIgnoreRestartSelf = true;

	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UTestBTTask_SetValuesWithLogs::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTSetValueTaskMemory* MyMemory = CastInstanceNodeMemory<FBTSetValueTaskMemory>(NodeMemory);
	MyMemory->EndFrameIdx = ExecutionTicks1 + FAITestHelpers::FramesCounter();
	MyMemory->EndFrameIdx2 = MyMemory->EndFrameIdx + ExecutionTicks2;

	OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(KeyName1, Value1);
	LogExecution(OwnerComp, LogIndex);

	if (ExecutionTicks1 == 0)
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(KeyName2, Value2);

		if (ExecutionTicks2 == 0)
		{
			return TaskResult;
		}
	}

	return EBTNodeResult::InProgress;
}
void UTestBTTask_SetValuesWithLogs::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTSetValueTaskMemory* MyMemory = CastInstanceNodeMemory<FBTSetValueTaskMemory>(NodeMemory);

	LogExecution(OwnerComp, LogTickIndex);

	if (FAITestHelpers::FramesCounter() >= MyMemory->EndFrameIdx)
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(KeyName2, Value2);
		MyMemory->EndFrameIdx = TNumericLimits<uint64>::Max();
	}

	if (FAITestHelpers::FramesCounter() >= MyMemory->EndFrameIdx2)
	{
		LogExecution(OwnerComp, LogFinished);
		FinishLatentTask(OwnerComp, TaskResult);
	}
}

uint16 UTestBTTask_SetValuesWithLogs::GetInstanceMemorySize() const
{
	return sizeof(FBTSetValueTaskMemory);
}

void UTestBTTask_SetValuesWithLogs::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTSetValueTaskMemory>(NodeMemory, InitType);
}

void UTestBTTask_SetValuesWithLogs::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTSetValueTaskMemory>(NodeMemory, CleanupType);
}

EBTNodeResult::Type UTestBTTask_SetValuesWithLogs::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (OnAbortKeyName.IsValid())
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Int>(OnAbortKeyName, OnAbortValue);
	}
	return EBTNodeResult::Aborted;	
}

void UTestBTTask_SetValuesWithLogs::LogExecution(UBehaviorTreeComponent& OwnerComp, int32 LogNumber)
{
	if (LogNumber >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}