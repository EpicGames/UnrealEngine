// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_TimerBasedLatent.h"
#include "Engine/World.h"
#include "MockAI_BT.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_TimerBasedLatent)

UTestBTTask_TimerBasedLatent::UTestBTTask_TimerBasedLatent()
{
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UTestBTTask_TimerBasedLatent::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTTimerBasedLatentTaskMemory* TaskMemory = CastInstanceNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory);
	TaskMemory->bIsAborting = false;

	LogExecution(OwnerComp, LogIndexExecuteStart);
	if (NumTicksExecuting == 0)
	{
		LogExecution(OwnerComp, LogIndexExecuteFinish);
		return LogResult;
	}

	if (const UWorld* World = OwnerComp.GetWorld())
	{
		const float DeltaTime = NumTicksExecuting * FAITestHelpers::TickInterval;
		World->GetTimerManager().SetTimer(TaskMemory->TimerHandle,
			FTimerDelegate::CreateWeakLambda(this, [&OwnerComp, TaskMemory, this]()
			{
				TaskMemory->TimerHandle.Invalidate();

				ensure(!TaskMemory->bIsAborting);
				LogExecution(OwnerComp, LogIndexExecuteFinish);
				FinishLatentTask(OwnerComp, LogResult);
			}), DeltaTime, false);
	}

	return EBTNodeResult::InProgress;
}

EBTNodeResult::Type UTestBTTask_TimerBasedLatent::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTTimerBasedLatentTaskMemory* TaskMemory = CastInstanceNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory);
	TaskMemory->bIsAborting = true;

	LogExecution(OwnerComp, LogIndexAbortStart);
	if (NumTicksAborting == 0)
	{
		LogExecution(OwnerComp, LogIndexAbortFinish);
		return EBTNodeResult::Aborted;
	}

	if (const UWorld* World = OwnerComp.GetWorld())
	{
		const float DeltaTime = NumTicksAborting * FAITestHelpers::TickInterval;
		World->GetTimerManager().SetTimer(TaskMemory->TimerHandle,
			FTimerDelegate::CreateWeakLambda(this, [&OwnerComp, TaskMemory, this]()
			{
				TaskMemory->TimerHandle.Invalidate();

				ensure(TaskMemory->bIsAborting);
				LogExecution(OwnerComp, LogIndexAbortFinish);
				FinishLatentAbort(OwnerComp);
			}), DeltaTime, false);
	}

	return EBTNodeResult::InProgress;
}

uint16 UTestBTTask_TimerBasedLatent::GetInstanceMemorySize() const
{
	return sizeof(FBTTimerBasedLatentTaskMemory);
}

void UTestBTTask_TimerBasedLatent::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory, InitType);
}

void UTestBTTask_TimerBasedLatent::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const EBTMemoryClear::Type CleanupType) const
{
	const FBTTimerBasedLatentTaskMemory* TaskMemory = CastInstanceNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory);
	if (TaskMemory->TimerHandle.IsValid())
	{
		if (const UWorld* World = OwnerComp.GetWorld())
		{
			World->GetTimerManager().ClearTimer(CastInstanceNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory)->TimerHandle);
		}
	}

	CleanupNodeMemory<FBTTimerBasedLatentTaskMemory>(NodeMemory, CleanupType);
}

void UTestBTTask_TimerBasedLatent::LogExecution(UBehaviorTreeComponent&, const int32 LogNumber) const
{
	if (LogNumber >= 0)
	{
		UMockAI_BT::ExecutionLog.Add(LogNumber);
	}
}
