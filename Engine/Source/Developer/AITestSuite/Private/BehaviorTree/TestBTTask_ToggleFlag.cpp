// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/TestBTTask_ToggleFlag.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBTTask_ToggleFlag)

UTestBTTask_ToggleFlag::UTestBTTask_ToggleFlag(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Log";
	TaskResult = EBTNodeResult::Succeeded;
	KeyName = TEXT("Bool1");
	NumToggles = 1;
}

EBTNodeResult::Type UTestBTTask_ToggleFlag::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	for (int i = 0; i < NumToggles; i++)
	{
		OwnerComp.GetBlackboardComponent()->SetValue<UBlackboardKeyType_Bool>(KeyName, !OwnerComp.GetBlackboardComponent()->GetValue<UBlackboardKeyType_Bool>(KeyName));
	}
	return TaskResult;
}
