// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockAI.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "Perception/AIPerceptionComponent.h"
#include "BrainComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MockAI)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
void FTestTickHelper::Tick(float DeltaTime)
{
	if (Owner.IsValid())
	{
		Owner->TickMe(DeltaTime);
	}
}

TStatId FTestTickHelper::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FTestTickHelper, STATGROUP_Tickables);
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UMockAI::UMockAI(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UWorld* World = FAITestHelpers::GetWorld();
		if (ensureMsgf(World != nullptr, TEXT("A world is required to spawn the associated test actor")))
		{
			FActorSpawnParameters SpawnParameters;
			SpawnParameters.Name = TEXT("MockAIActor");
			SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
			SpawnParameters.ObjectFlags = RF_Transient;
			Actor = World->SpawnActor<AActor>(SpawnParameters);
		}
	}
}

UMockAI::~UMockAI()
{
	TickHelper.Owner.Reset();

	if (Actor != nullptr)
	{
		Actor->Destroy();
	}
}

void UMockAI::SetEnableTicking(bool bShouldTick)
{
	if (bShouldTick)
	{
		TickHelper.Owner = this;
	}
	else
	{
		TickHelper.Owner = nullptr;
	}
}

void UMockAI::UseBlackboardComponent()
{
	BBComp = NewObject<UBlackboardComponent>(Actor);
}

void UMockAI::UsePerceptionComponent()
{
	PerceptionComp = NewObject<UAIPerceptionComponent>(Actor);
}

void UMockAI::TickMe(float DeltaTime)
{
	if (BBComp)
	{
		BBComp->TickComponent(DeltaTime, ELevelTick::LEVELTICK_All, nullptr);
	}

	if (PerceptionComp)
	{
		PerceptionComp->TickComponent(DeltaTime, ELevelTick::LEVELTICK_All, nullptr);
	}
	
	if (BrainComp)
	{
		BrainComp->TickComponent(DeltaTime, ELevelTick::LEVELTICK_All, nullptr);
	}
}


