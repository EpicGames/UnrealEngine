// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockAI_BT.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MockAI_BT)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
TArray<int32> UMockAI_BT::ExecutionLog;

UMockAI_BT::UMockAI_BT(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UseBlackboardComponent();
		UseBrainComponent<UBehaviorTreeComponent>();

		BTComp = Cast<UBehaviorTreeComponent>(BrainComp);

		// We want our component to be ticked manually by the test framework, never by the TickFunction
		BTComp->PrimaryComponentTick.bCanEverTick = false;
		BTComp->RegisterComponent();

		// Test running in a game world will get initialized automatically by
		// the previous call to RegisterComponent. Otherwise, we initialize it
		// manually.
		if (!BTComp->HasBeenInitialized() && BTComp->bWantsInitializeComponent)
		{
			BTComp->InitializeComponent();
		}
	}
}

bool UMockAI_BT::IsRunning() const
{
	return BTComp && BTComp->IsRunning() && BTComp->GetRootTree();
}

void UMockAI_BT::RunBT(UBehaviorTree& BTAsset, EBTExecutionMode::Type RunType)
{
	if (BTAsset.BlackboardAsset)
	{
		BBComp->InitializeBlackboard(*BTAsset.BlackboardAsset);
	}
	BBComp->CacheBrainComponent(*BTComp);
	BTComp->CacheBlackboardComponent(BBComp);

	UWorld* World = FAITestHelpers::GetWorld();

	BBComp->RegisterComponentWithWorld(World);
	BTComp->RegisterComponentWithWorld(World);

	BTComp->StartTree(BTAsset, RunType);
}

