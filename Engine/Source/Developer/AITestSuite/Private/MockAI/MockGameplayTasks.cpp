// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockGameplayTasks.h"
#include "TestLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MockGameplayTasks)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UMockTask_Log::UMockTask_Log(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Logger(nullptr)
	, bShoudEndAsPartOfActivation(false)
{

}

UMockTask_Log* UMockTask_Log::CreateTask(IGameplayTaskOwnerInterface& TaskOwner, FTestLogger<int32>& InLogger, const FGameplayResourceSet& Resources, uint8 Priority)
{
	UMockTask_Log* Task = NewTask<UMockTask_Log>(TaskOwner);
	if (Task)
	{
		Task->Logger = &InLogger;
		Task->RequiredResources = Resources;
		Task->ClaimedResources = Resources;
		Task->Priority = Priority;
	}
	return Task;
}

void UMockTask_Log::Activate()
{
	if (Logger)
	{
		Logger->Log(ETestTaskMessage::Activate);
	}
	Super::Activate();
	if (bShoudEndAsPartOfActivation)
	{
		EndTask();
	}
}

void UMockTask_Log::OnDestroy(bool bInOwnerFinished)
{
	if (Logger)
	{
		Logger->Log(ETestTaskMessage::Ended);
	}
	Super::OnDestroy(bInOwnerFinished);
}

void UMockTask_Log::TickTask(float DeltaTime)
{
	if (Logger)
	{
		Logger->Log(ETestTaskMessage::Tick);
	}
	Super::TickTask(DeltaTime);
}

void UMockTask_Log::ExternalConfirm(bool bEndTask)
{
	if (Logger)
	{
		Logger->Log(ETestTaskMessage::ExternalConfirm);
	}
	Super::ExternalConfirm(bEndTask);
}

void UMockTask_Log::ExternalCancel()
{
	if (Logger)
	{
		Logger->Log(ETestTaskMessage::ExternalCancel);
	}
	Super::ExternalCancel();
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//

