// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"

DEFINE_LOG_CATEGORY(LogAITestSuite);
DEFINE_LOG_CATEGORY(LogBehaviorTreeTest);
DEFINE_LOG_CATEGORY(LogTestableEnsures);


namespace UE::AITestSuite
{
	bool bBreakOnTestFail = false;
	FAutoConsoleVariableRef CVarBreakOnTestFail(TEXT("test.BreakOnTestFail")
		, bBreakOnTestFail
		, TEXT("Enables AITestSuite-implemented tests to cause a debug break when tests fail. Used when fixing test offenders or during test-driven development.")
		, ECVF_Default);

	void ConditionallyBreakOnTestFail()
	{
		if (bBreakOnTestFail)
		{
			UE_DEBUG_BREAK();
		}
	}
}

namespace UE::AITestSuite
{
	int32 TestsInProgress = 0;
} // namespace UE::AITestSuite

namespace FAITestHelpers
{
	uint64 UpdatesCounter = 0;
	
	void UpdateFrameCounter()
	{
		static uint64 PreviousFramesCounter = GFrameCounter;
		if (PreviousFramesCounter != GFrameCounter)
		{
			++UpdatesCounter;
			PreviousFramesCounter = GFrameCounter;
		}
	}

	uint64 FramesCounter()
	{
		return UpdatesCounter;
	}
}

bool FAITestCommand_WaitSeconds::Update()
{
	const double NewTime = FPlatformTime::Seconds();
	if (NewTime - StartTime >= Duration)
	{
		return true;
	}
	return false;
}

bool FAITestCommand_WaitOneTick::Update()
{
	if (bAlreadyRun == false)
	{
		bAlreadyRun = true;
		return true;
	}
	return false;
}

bool FAITestCommand_SetUpTest::Update()
{
	return AITest && AITest->SetUp();
}

bool FAITestCommand_PerformTest::Update()
{
	return AITest == nullptr || AITest->Update();
}

bool FAITestCommand_VerifyTestResults::Update()
{
	if (AITest)
	{
		AITest->VerifyLatentResults();
	}
	// signal "done"
	return true; 
}

bool FAITestCommand_TearDownTest::Update()
{
	if (AITest)
	{
		AITest->TearDown();
		delete AITest;
		AITest = nullptr;
	}
	return true;
}

namespace FAITestHelpers
{
	UWorld* GetWorld()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			return GWorld;
		}
#endif // WITH_EDITOR
		return GEngine->GetWorldContexts()[0].World();
	}
}

//----------------------------------------------------------------------//
// FAITestBase
//----------------------------------------------------------------------//
FAITestBase::~FAITestBase()
{
	check(bTearedDown && "Super implementation of TearDown not called!");
}

void FAITestBase::AddAutoDestroyObject(UObject& ObjectRef)
{
	ObjectRef.AddToRoot();
	SpawnedObjects.Add(&ObjectRef);
}

UWorld& FAITestBase::GetWorld() const
{
	UWorld* World = FAITestHelpers::GetWorld();
	check(World);
	return *World;
}

void FAITestBase::TearDown()
{
	bTearedDown = true;
	for (auto AutoDestroyedObject : SpawnedObjects)
	{
		AutoDestroyedObject->RemoveFromRoot();
		AutoDestroyedObject->MarkAsGarbage();
	}
	SpawnedObjects.Reset();
}
