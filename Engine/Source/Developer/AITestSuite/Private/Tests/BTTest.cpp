// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "BTBuilder.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/Decorators/BTDecorator_Cooldown.h"
#include "BehaviorTree/Decorators/BTDecorator_ForceSuccess.h"
#include "BehaviorTree/Decorators/BTDecorator_TimeLimit.h"
#include "BehaviorTree/TestBTDecorator_CantExecute.h"
#include "BehaviorTree/TestBTTask_TimerBasedLatent.h"
#include "BehaviorTree/TestBTTask_RunBehavior.h"
#include "Engine/World.h"
#include "MockAI_BT.h"
#include "TimerManager.h"

#define LOCTEXT_NAMESPACE "AITestSuite_BTTest"

//----------------------------------------------------------------------//
// FAITest_SimpleBT
//----------------------------------------------------------------------//
struct FAITest_SimpleBT : public FAITestBase
{
	TArray<int32> ExpectedResult;
	UBehaviorTree* BTAsset;
	UMockAI_BT* AIBTUser;
	bool bUseSystemTicking;

	FAITest_SimpleBT()
	{
		bUseSystemTicking = false;

		BTAsset = &FBTBuilder::CreateBehaviorTree();
		if (BTAsset)
		{
			AddAutoDestroyObject(*BTAsset);
		}
	}

	virtual bool SetUp() override
	{
		FAITestBase::SetUp();

		AIBTUser = NewAutoDestroyObject<UMockAI_BT>();

		UMockAI_BT::ExecutionLog.Reset();

		if (AIBTUser && BTAsset)
		{
			AIBTUser->RunBT(*BTAsset, EBTExecutionMode::SingleRun);
			AIBTUser->SetEnableTicking(bUseSystemTicking);
			return true;
		}
		return false;
	}

	virtual bool Update() override
	{
		FAITestHelpers::UpdateFrameCounter();

		if (AIBTUser != nullptr)
		{
			// Some tests rely on timers, so we need to manually tick the world timer manager
			// if it was not ticked in the frame (e.g., the editor world not ticking if viewport is not set to 'realtime')
			FTimerManager& TimerManager = GetWorld().GetTimerManager();
			if (!TimerManager.HasBeenTickedThisFrame())
			{
				TimerManager.Tick(FAITestHelpers::TickInterval);
			}

			if (bUseSystemTicking == false)
			{
				AIBTUser->TickMe(FAITestHelpers::TickInterval);
			}

			if (AIBTUser->IsRunning())
			{
				return false;
			}
		}

		VerifyResults();
		return true;
	}

	bool VerifyResults()
	{
		const bool bMatch = (ExpectedResult == UMockAI_BT::ExecutionLog);
		//ensure(bMatch && "VerifyResults failed!");
		if (!bMatch)
		{
			FString DescriptionResult;
			for (int32 Idx = 0; Idx < UMockAI_BT::ExecutionLog.Num(); Idx++)
			{
				DescriptionResult += TTypeToString<int32>::ToString(UMockAI_BT::ExecutionLog[Idx]);
				if (Idx < (UMockAI_BT::ExecutionLog.Num() - 1))
				{
					DescriptionResult += TEXT(", ");
				}
			}

			FString DescriptionExpected;
			for (int32 Idx = 0; Idx < ExpectedResult.Num(); Idx++)
			{
				DescriptionExpected += TTypeToString<int32>::ToString(ExpectedResult[Idx]);
				if (Idx < (ExpectedResult.Num() - 1))
				{
					DescriptionExpected += TEXT(", ");
				}
			}

			UE_LOGF(LogBehaviorTreeTest, Error, "Test scenario failed to produce expected results!\nExecution log: %ls\nExpected values: %ls", *DescriptionResult, *DescriptionExpected);
		}
		return bMatch;
	}
};

//----------------------------------------------------------------------//
// TESTS 
//----------------------------------------------------------------------//
struct FAITest_BTBasicSelector : public FAITest_SimpleBT
{
	FAITest_BTBasicSelector()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			FBTBuilder::AddTask(CompNode, 1, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode);
			}

			FBTBuilder::AddTask(CompNode, 2, EBTNodeResult::Succeeded, 2);

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTBasicSelector, "System.AI.Behavior Trees.Composite node: selector")

struct FAITest_BTSearchSequenceForceSuccessTask : public FAITest_SimpleBT
{
	FAITest_BTSearchSequenceForceSuccessTask()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, Task1Execute, EBTNodeResult::Succeeded);

			// First decorator will deny execution of Task 2 but ForceSuccess will only sequence to continue
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode);
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(CompNode);
			}

			FBTBuilder::AddTask(CompNode, Task3Execute, EBTNodeResult::Failed);

			FBTBuilder::AddTask(CompNode, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task3Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSearchSequenceForceSuccessTask, "System.AI.Behavior Trees.Composite node: search sequence with force success child task")

struct FAITest_BTDeepSearchSequenceForceSuccessSelf : public FAITest_SimpleBT
{
	FAITest_BTDeepSearchSequenceForceSuccessSelf()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
		};

		UBTCompositeNode& Selector = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(Selector);
			{
				UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
				{
					// Force success must allow sequence to continue even if the search root node is not its associate node or its immediate parent.
					FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(MainSequence);
					FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(MainSequence);

					FBTBuilder::AddTask(InnerSequence, Task1Execute, EBTNodeResult::Succeeded);
				}

				FBTBuilder::AddTask(MainSequence, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(Task2Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDeepSearchSequenceForceSuccessSelf, "System.AI.Behavior Trees.Composite node: deep search sequence with force success self")

struct FAITest_BTSearchParentNotAffectedByForceSuccessChild : public FAITest_SimpleBT
{
	FAITest_BTSearchParentNotAffectedByForceSuccessChild()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& MiddleSequence = FBTBuilder::AddSequence(MainSequence);
			{
				UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MiddleSequence);
				{
					FBTBuilder::AddTask(InnerSequence, Task1Execute, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(InnerSequence);
						FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(InnerSequence);
					}

					FBTBuilder::AddTask(MiddleSequence, Task2Execute, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(MiddleSequence);
					}
				}
				
				FBTBuilder::AddTask(MiddleSequence, Task3Execute, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task4Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSearchParentNotAffectedByForceSuccessChild, "System.AI.Behavior Trees.Composite node: search in parent node not affected by force success child")

struct FAITest_BTDecoratorBlueprint : public FAITest_SimpleBT
{
	FAITest_BTDecoratorBlueprint()
	{
		enum
		{
			DecoratorBlueprintCalculate = 1,
			DecoratorBlueprintBecomeRelevant,
			DecoratorBlueprintCeaseRelevant,
			Task1Execute,
			Task2Execute,
			Task3Execute,
		};

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode1 = FBTBuilder::AddSequence(CompNode);
			FBTBuilder::WithDecoratorBlueprint(CompNode, EBTFlowAbortMode::Both, EBPConditionType::TrueCondition, DecoratorBlueprintBecomeRelevant, DecoratorBlueprintCeaseRelevant, DecoratorBlueprintCalculate);
			{
				// Task1
				FBTBuilder::AddTask(CompNode1, Task1Execute, EBTNodeResult::Succeeded);

				// Task2
				FBTBuilder::AddTask(CompNode1, Task2Execute, EBTNodeResult::Succeeded, /*ExecutionTicks*/1);
			}
			// Task3
			FBTBuilder::AddTask(CompNode, Task3Execute, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(DecoratorBlueprintBecomeRelevant/*2*/);
		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(Task1Execute/*4*/);
		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(Task2Execute/*5*/);
		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(DecoratorBlueprintCeaseRelevant/*3*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorBlueprint, "System.AI.Behavior Trees.Decorator: blueprint")

struct FAITest_BTDecoratorBlueprintNoConditionImpl : public FAITest_SimpleBT
{
	FAITest_BTDecoratorBlueprintNoConditionImpl()
	{
		enum
		{
			DecoratorBlueprintCalculate = 1,
			DecoratorBlueprintBecomeRelevant,
			DecoratorBlueprintCeaseRelevant,
			Task1Execute,
			Task2Execute,
			Task3Execute,
		};

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			// Task1
			FBTBuilder::AddTask(CompNode, Task1Execute, EBTNodeResult::Succeeded);
			FBTBuilder::WithDecoratorBlueprint(CompNode, EBTFlowAbortMode::LowerPriority, EBPConditionType::NoCondition, DecoratorBlueprintBecomeRelevant, DecoratorBlueprintCeaseRelevant, DecoratorBlueprintCalculate, TEXT("Bool1"));

			// Task2
			FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::Succeeded, TEXT("Bool1"));
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));

			// Task3
			FBTBuilder::AddTask(CompNode, Task3Execute, EBTNodeResult::Succeeded, /*ExecutionTicks*/1);
		}

		ExpectedResult.Add(DecoratorBlueprintCalculate/*1*/);
		ExpectedResult.Add(DecoratorBlueprintBecomeRelevant/*2*/);
		ExpectedResult.Add(Task3Execute/*6*/);
		ExpectedResult.Add(DecoratorBlueprintCeaseRelevant/*3*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorBlueprintNoConditionImpl, "System.AI.Behavior Trees.Decorator: blueprint no condition impl")

struct FAITest_BTDecoratorForceSuccessTaskFailed : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessTaskFailed()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::AddTask(InnerSequence, Task2Execute, EBTNodeResult::Failed);
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(InnerSequence);

				FBTBuilder::AddTask(InnerSequence, Task3Execute, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
		ExpectedResult.Add(Task3Execute);
		ExpectedResult.Add(Task4Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessTaskFailed, "System.AI.Behavior Trees.Decorator: force success task failed self")

struct FAITest_BTDecoratorForceSuccessTaskFailedParentComposite : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessTaskFailedParentComposite()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::WithDecoratorBlackboard(MainSequence, EArithmeticKeyOperation::NotEqual, /*IntValue*/1, EBTFlowAbortMode::Self, EBTBlackboardRestart::ResultChange);

				FBTBuilder::AddTaskValuesChangedWithLogs(InnerSequence, Task2Execute, EBTNodeResult::Succeeded, /*IntValue1*/1, /*IntValue2*/INDEX_NONE);
				{
					FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(InnerSequence);
				}
				
				FBTBuilder::AddTask(InnerSequence, Task3Execute, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessTaskFailedParentComposite, "System.AI.Behavior Trees.Decorator: force success task failed parent composite")

struct FAITest_BTDecoratorForceSuccessCompositeFailed : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessCompositeFailed()
	{
		enum
		{
			Task1Execute=1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(MainSequence);
				FBTBuilder::WithDecoratorBlackboard(MainSequence, EArithmeticKeyOperation::NotEqual, /*IntValue*/1, EBTFlowAbortMode::Self, EBTBlackboardRestart::ResultChange);

				FBTBuilder::AddTaskValuesChangedWithLogs(InnerSequence, Task2Execute, EBTNodeResult::Succeeded, /*IntValue1*/1, /*IntValue2*/INDEX_NONE);

				FBTBuilder::AddTask(InnerSequence, Task3Execute, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
		ExpectedResult.Add(Task4Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessCompositeFailed, "System.AI.Behavior Trees.Decorator: force success composite failed self")

struct FAITest_BTDecoratorForceSuccessCompositeFailedFromLastChild : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessCompositeFailedFromLastChild()
	{
		enum
		{
			Task1Execute=1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(MainSequence);
				FBTBuilder::WithDecoratorBlackboard(MainSequence, EArithmeticKeyOperation::NotEqual, /*IntValue*/1, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange);

				FBTBuilder::AddTask(InnerSequence, Task2Execute, EBTNodeResult::Succeeded);

				FBTBuilder::AddTaskValuesChangedWithLogs(InnerSequence, Task3Execute, EBTNodeResult::Succeeded, /*IntValue1*/1, /*IntValue2*/INDEX_NONE);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
		ExpectedResult.Add(Task3Execute);
		ExpectedResult.Add(Task4Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessCompositeFailedFromLastChild, "System.AI.Behavior Trees.Decorator: force success composite failed self from last child")

struct FAITest_BTDecoratorForceSuccessCompositeFailedChild : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessCompositeFailedChild()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(MainSequence);

				FBTBuilder::AddTask(InnerSequence, Task2Execute, EBTNodeResult::Failed);

				FBTBuilder::AddTask(InnerSequence, Task3Execute, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
		ExpectedResult.Add(Task4Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessCompositeFailedChild, "System.AI.Behavior Trees.Decorator: force success composite failed child task")

struct FAITest_BTDecoratorForceSuccessCompositeFailedChildFromSubTree : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessCompositeFailedChildFromSubTree()
	{
		enum
		{
			ChildTask1Execute = 1,
			ChildTask2Execute,
			Task1Execute,
			Task2Execute,
		};

		// Subtree that will trigger the abort condition (i.e, Bool1 set) of the composite node in the parent tree
		UBehaviorTree* ChildAsset1 = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset1)
		{
			AddAutoDestroyObject(*ChildAsset1);
			UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*ChildAsset1);
			{
				FBTBuilder::AddTask(CompNode, ChildTask1Execute, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::InProgress, TEXT("Bool1"));
				FBTBuilder::AddTask(CompNode, ChildTask2Execute, EBTNodeResult::Succeeded);
			}
		}

		// Main tree with a composite that aborts on key change but should succeed with ForceSuccess decorator
		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);
			FBTBuilder::AddTaskSubtree(MainSequence, ChildAsset1);
			{
				FBTBuilder::WithDecoratorBlackboard(MainSequence, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(MainSequence);
			}
			FBTBuilder::AddTask(MainSequence, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(ChildTask1Execute);
		ExpectedResult.Add(Task2Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessCompositeFailedChildFromSubTree, "System.AI.Behavior Trees.Decorator: force success composite failed from subtree task")

struct FAITest_BTDecoratorForceSuccessTaskFailedSibling : public FAITest_SimpleBT
{
	FAITest_BTDecoratorForceSuccessTaskFailedSibling()
	{
		enum
		{
			Task1Execute = 1,
			Task2Execute,
			Task3Execute,
			Task4Execute,
		};

		UBTCompositeNode& MainSequence = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(MainSequence, Task1Execute, EBTNodeResult::Succeeded);

			UBTCompositeNode& InnerSequence = FBTBuilder::AddSequence(MainSequence);
			{
				FBTBuilder::AddTask(InnerSequence, Task2Execute, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(InnerSequence);
				}

				FBTBuilder::AddTask(InnerSequence, Task3Execute, EBTNodeResult::Failed);
			}

			FBTBuilder::AddTask(MainSequence, Task4Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task2Execute);
		ExpectedResult.Add(Task3Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDecoratorForceSuccessTaskFailedSibling, "System.AI.Behavior Trees.Decorator: force success task failed sibling task")

struct FAITest_BTBasicParallelWait : public FAITest_SimpleBT
{
	FAITest_BTBasicParallelWait()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::WaitForBackground);
			{
				FBTBuilder::AddTaskLogFinish(CompNode2, 0, 10, EBTNodeResult::Succeeded, 6);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTaskLogFinish(CompNode3, 1, 11, EBTNodeResult::Succeeded, 3);
					FBTBuilder::AddTaskLogFinish(CompNode3, 2, 12, EBTNodeResult::Succeeded, 3);
					FBTBuilder::AddTaskLogFinish(CompNode3, 3, 13, EBTNodeResult::Succeeded, 3);
				}
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
			ExpectedResult.Add(1);
			ExpectedResult.Add(11);
			ExpectedResult.Add(2);
		ExpectedResult.Add(10);
			ExpectedResult.Add(12);
			ExpectedResult.Add(3);
			ExpectedResult.Add(13);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTBasicParallelWait, "System.AI.Behavior Trees.Composite node: parallel (wait)")

struct FAITest_BTBasicParallelAbort : public FAITest_SimpleBT
{
	FAITest_BTBasicParallelAbort()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::AbortBackground);
			{
				FBTBuilder::AddTaskLogFinish(CompNode2, 0, 10, EBTNodeResult::Succeeded, 6);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTaskLogFinish(CompNode3, 1, 11, EBTNodeResult::Succeeded, 4);
					FBTBuilder::AddTaskLogFinish(CompNode3, 2, 12, EBTNodeResult::Succeeded, 4);
					FBTBuilder::AddTaskLogFinish(CompNode3, 3, 13, EBTNodeResult::Succeeded, 4);
				}
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
			ExpectedResult.Add(1);
			ExpectedResult.Add(11);
			ExpectedResult.Add(2);
		ExpectedResult.Add(10);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTBasicParallelAbort, "System.AI.Behavior Trees.Composite node: parallel (abort)")

struct FAITest_BTCompositeDecorator : public FAITest_SimpleBT
{
	FAITest_BTCompositeDecorator()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);

			FBTBuilder::AddTask(CompNode, 1, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::None, TEXT("Bool1"));
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::None, TEXT("Bool2"));
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool3"));
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::None, TEXT("Bool4"));

				TArray<FBTDecoratorLogic>& CompositeOps = CompNode.Children.Last().DecoratorOps;
				CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Or, 3));
					CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, 0));
					CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Not, 1));
						CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::And, 2));
							CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, 1));
							CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, 2));
					CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, 3));
			}

			FBTBuilder::AddTask(CompNode, 2, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCompositeDecorator, "System.AI.Behavior Trees.Composite decorator")

struct FAITest_BTCompositeChildrenDecoratorsNotUnregistered : public FAITest_SimpleBT
{
	/**
	 *                           +-------+
	 *                           |root   |
	 *                           +---+---+
	 *                               |
	 *                      +--------+--------+
	 *                      |Sequence      (0)|
	 *                      +--------+--------+
	 *                               |
	 *          +--------------------+--------------------+
	 *          |                                         |
	 * +--------+--------+                       +--------+--------+
	 * |Set Int=1     (1)|                       |Selector      (2)|
	 * +-----------------+                       +--------+--------+
	 *                                                    |
	 *                                +-------------------+--------------------+--------------------+
	 *                                |                                        |                    |
	 *                       +--------+--------+                      +--------+--------+  +--------+--------+
	 *                       |Selector      (3)|                      |BBDec: Int==2(10)|  |Set Int=2    (12)|
	 *                       +--------+--------+                      +-----------------+  +-----------------+
	 *                                |                               |Task 3       (11)|
	 *           +-----------------------------------------+          +-----------------+
	 *           |                    |                    |
	 *  +--------+--------+  +--------+--------+  +--------+--------+
	 *  |BBDec:Int==11 (4)|  |BBDec:Int==12 (6)|  |BBDec: Int==1 (8)|
	 *  +-----------------+  +-----------------+  +-----------------+
	 *  |Task1         (5)|  |Task2         (7)|  |   Set Int=0  (9)|
	 *  +-----------------+  +-----------------+  +-----------------+
	 *
	 */
	FAITest_BTCompositeChildrenDecoratorsNotUnregistered()
	{
		UBTCompositeNode& CompNode0 = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTaskValueChange(CompNode0, 1, EBTNodeResult::Succeeded, TEXT("Int"));

			UBTCompositeNode& CompNode = FBTBuilder::AddSelector(CompNode0);
			{
				UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
				{
					FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode2, EArithmeticKeyOperation::Equal, 11, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
							TEXT("Int"), 110 /*LogIndexBecomeRelevant*/, 119 /*LogIndexCeaseRelevant*/);
					}

					FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode2, EArithmeticKeyOperation::Equal, 12, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
							TEXT("Int"), 120 /*LogIndexBecomeRelevant*/, 129 /*LogIndexCeaseRelevant*/);
					}

					FBTBuilder::AddTaskValueChange(CompNode2, 0, EBTNodeResult::Succeeded, TEXT("Int"));
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode2, EArithmeticKeyOperation::Equal, 1, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
							TEXT("Int"), 130 /*LogIndexBecomeRelevant*/, 139 /*LogIndexCeaseRelevant*/);
					}
				}

				FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::Equal, 2, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
						TEXT("Int"), 200 /*LogIndexBecomeRelevant*/, 299 /*LogIndexCeaseRelevant*/);
				}

				FBTBuilder::AddTaskValueChange(CompNode, 2, EBTNodeResult::Succeeded, TEXT("Int"));
			}
		}

		ExpectedResult.Add(110);
		ExpectedResult.Add(120);
		ExpectedResult.Add(130);
		ExpectedResult.Add(200);
		ExpectedResult.Add(3);
		ExpectedResult.Add(119);
		ExpectedResult.Add(129);
		ExpectedResult.Add(139);
		ExpectedResult.Add(299);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCompositeChildrenDecoratorsNotUnregistered, "System.AI.Behavior Trees.Composite children decorators not unregistered")

struct FAITest_BTCompositeFailedDecoratorUnregistersChildren : public FAITest_SimpleBT
{
	/**
	 *                           +-------+
	 *                           |root   |
	 *                           +---+---+
	 *                               |
	 *                      +--------+--------+
	 *                      |Sequence      (0)|
	 *                      +--------+--------+
	 *                               |
	 *          +--------------------+--------------------+
	 *          |                                         |
	 * +--------+--------+                       +--------+--------+
	 * |Set Int=1     (1)|                       |Selector      (2)|
	 * +-----------------+                       +--------+--------+
	 *                                                    |
	 *                                +-------------------+--------------------+--------------------+
	 *                                |                                        |                    |
	 *                       +--------+--------+                      +--------+--------+  +--------+--------+
	 *                       |BBDec: Int==1 (3)|                      |BBDec: Int==2(10)|  |Set Int=2    (12)|
	 *                       +-----------------+                      +-----------------+  +-----------------+
	 *                       |Selector      (4)|                      |Task 3       (11)|
	 *                       +--------+--------+                      +-----------------+
	 *                                |
	 *           +-----------------------------------------+
	 *           |                    |                    |
	 *  +--------+--------+  +--------+--------+  +--------+--------+
	 *  |BBDec:Int==11 (5)|  |BBDec:Int==12 (7)|  |   Set Int=0  (9)|
	 *  +-----------------+  +-----------------+  +-----------------+
	 *  |Task1         (6)|  |Task2         (8)|
	 *  +-----------------+  +-----------------+
	 *
	 *
	 */
	FAITest_BTCompositeFailedDecoratorUnregistersChildren()
	{
		UBTCompositeNode& CompNode0 = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTaskValueChange(CompNode0, 1, EBTNodeResult::Succeeded, TEXT("Int"));
			
			UBTCompositeNode& CompNode = FBTBuilder::AddSelector(CompNode0);
			{
				UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::Equal, 1, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
						TEXT("Int"), 100 /*LogIndexBecomeRelevant*/, 199 /*LogIndexCeaseRelevant*/);

					FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode2, EArithmeticKeyOperation::Equal, 11, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
							TEXT("Int"), 110 /*LogIndexBecomeRelevant*/, 119 /*LogIndexCeaseRelevant*/);
					}

					FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode2, EArithmeticKeyOperation::Equal, 12, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange,
							TEXT("Int"), 120 /*LogIndexBecomeRelevant*/, 129 /*LogIndexCeaseRelevant*/);
					}

					FBTBuilder::AddTaskValueChange(CompNode2, 0, EBTNodeResult::Succeeded, TEXT("Int"));
				}
								
				FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::Equal, 2, EBTFlowAbortMode::Both, EBTBlackboardRestart::ResultChange, TEXT("Int"), 200 /*LogIndexBecomeRelevant*/, 299 /*LogIndexCeaseRelevant*/);
				}

				FBTBuilder::AddTaskValueChange(CompNode, 2, EBTNodeResult::Succeeded, TEXT("Int"));
			}
		}

		ExpectedResult.Add(100);
		ExpectedResult.Add(110);
		ExpectedResult.Add(120);
		ExpectedResult.Add(119);
		ExpectedResult.Add(129);
		ExpectedResult.Add(200);
		ExpectedResult.Add(3);
		ExpectedResult.Add(199);
		ExpectedResult.Add(299);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCompositeFailedDecoratorUnregistersChildren, "System.AI.Behavior Trees.Composite failed decorator unregisters child nodes")

struct FAITest_BTTimeLimitDecorator : FAITest_SimpleBT
{
	FAITest_BTTimeLimitDecorator()
	{
		enum
		{
			Task1Execute = 1,
			Task1Tick,
			Task2Execute,
		};
		
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, Task1Execute, EBTNodeResult::Succeeded, /*ExecutionTicks*/ 5, Task1Tick);
			{
				constexpr float NumTicks = 2.5f;
				FBTBuilder::WithDecorator<UBTDecorator_TimeLimit>(CompNode).TimeLimit = NumTicks * FAITestHelpers::TickInterval;
			}

			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task1Tick);
		ExpectedResult.Add(Task1Tick);
		ExpectedResult.Add(Task2Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTTimeLimitDecorator, "System.AI.Behavior Trees.TimeLimit decorator")

struct FAITest_BTTimeLimitInCompositeDecorator : FAITest_SimpleBT
{
	FAITest_BTTimeLimitInCompositeDecorator()
	{
		enum
		{
			Task1Execute = 1,
			Task1Tick,
			Task2Execute,
		};
		
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, Task1Execute, EBTNodeResult::Succeeded, /*ExecutionTicks*/ 5, Task1Tick);
			{
				constexpr float NumTicks = 2.5f;
				FBTBuilder::WithDecorator<UBTDecorator_TimeLimit>(CompNode).TimeLimit = NumTicks * FAITestHelpers::TickInterval;

				TArray<FBTDecoratorLogic>& CompositeOps = CompNode.Children.Last().DecoratorOps;
				CompositeOps.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, 0));
			}

			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(Task1Execute);
		ExpectedResult.Add(Task1Tick);
		ExpectedResult.Add(Task1Tick);
		ExpectedResult.Add(Task2Execute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTTimeLimitInCompositeDecorator, "System.AI.Behavior Trees.TimeLimit in Composite decorator")

/* All BTAbortingDuringService/BTAbortingDuringTaskService come from UDN case 00317509*/
struct FAITest_BTAbortingDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringServiceTick()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
				FBTBuilder::WithServiceLog(CompNode2, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringServiceTick, "System.AI.Behavior Trees.Abort: during service tick")

struct FAITest_BTRestartingTheAbortedBranch : public FAITest_SimpleBT
{
	FAITest_BTRestartingTheAbortedBranch()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 2/*NumOfToggles*/);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
											    NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRestartingTheAbortedBranch, "System.AI.Behavior Trees.Toggle: restart the aborted branch")

struct FAITest_BTAbortingTheRestarted : public FAITest_SimpleBT
{
	FAITest_BTAbortingTheRestarted()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 3/*NumOfToggles*/);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingTheRestarted, "System.AI.Behavior Trees.Toggle: reabort the restarted branch")

struct FAITest_BTRestartingTheReaborted : public FAITest_SimpleBT
{
	FAITest_BTRestartingTheReaborted()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 4/*NumOfToggles*/);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRestartingTheReaborted, "System.AI.Behavior Trees.Toggle: restart the reaborted branch")

struct FAITest_BTRestartingTheAbortedBranchDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTRestartingTheAbortedBranchDuringServiceTick()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 4, EBTNodeResult::Succeeded);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, TEXT("Bool1") /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, NAME_None /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, TEXT("Bool1") /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, NAME_None /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);

					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRestartingTheAbortedBranchDuringServiceTick, "System.AI.Behavior Trees.Toggle: restart the aborted branch during service tick")

struct FAITest_BTRestartTheAbortedBranchDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTRestartTheAbortedBranchDuringServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 1/*NumOfToggles*/);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool1") /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);

					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRestartTheAbortedBranchDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Toggle: restart the aborted branch during service cease relevant")

struct FAITest_BTRestartTheReabortedBranchDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTRestartTheReabortedBranchDuringServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 1/*NumOfToggles*/);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool1") /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool1") /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, -1 /*ActivationIndex*/, -1 /*DeactivationIndex*/, -1 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool1") /*CeaseRelevantBoolKeyName*/, true /*bToggleValues*/);

					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRestartTheReabortedBranchDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Toggle: restart the reaborted branch during service cease relevant")

struct FAITest_BTStartBranchDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTStartBranchDuringServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 5, EBTNodeResult::Succeeded);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2"));

					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 1/*NumOfToggles*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
				}
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStartBranchDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Toggle: start new branch during service cease relevant")

struct FAITest_BTStartBranchDuringServiceCeaseRelevant2 : public FAITest_SimpleBT
{
	FAITest_BTStartBranchDuringServiceCeaseRelevant2()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 6/*LogIndex*/, EBTNodeResult::Succeeded, 2/* ExecutionTicks*/);
				FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool1"));

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 5/*LogIndex*/, EBTNodeResult::Succeeded);
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2"));

					FBTBuilder::AddTaskToggleFlag(CompNode3, EBTNodeResult::InProgress, TEXT("Bool1"), 1/*NumOfToggles*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, 1 /*ActivationIndex*/, 2 /*DeactivationIndex*/, 3 /*TickIndex*/, NAME_None /*TickBoolKeyName*/, false /*bCallTickOnSearchStart*/,
						NAME_None /*BecomeRelevantBoolKeyeyName*/, TEXT("Bool2") /*CeaseRelevantBoolKeyName*/);
				}
			}
			FBTBuilder::AddTask(CompNode, 7/*LogIndex*/, EBTNodeResult::Succeeded);
		}
			
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStartBranchDuringServiceCeaseRelevant2, "System.AI.Behavior Trees.Toggle: start new branch during service cease relevant (low prio)")

struct FAITest_BTAbortingDuringServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringServiceBecomeRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
				FBTBuilder::WithServiceLog(CompNode2, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringServiceBecomeRelevant, "System.AI.Behavior Trees.Abort: during service becomes relevant")

struct FAITest_BTAbortingDuringServiceBecomeRelevantTaskTicking : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringServiceBecomeRelevantTaskTicking()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
				FBTBuilder::WithServiceLog(CompNode2, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringServiceBecomeRelevantTaskTicking, "System.AI.Behavior Trees.Abort: during service becomes relevant when task continue to tick for a longer time")

struct FAITest_BTAbortingDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
					FBTBuilder::WithServiceLog(CompNode3, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, NAME_None, TEXT("Bool1"));

					FBTBuilder::AddTask(CompNode3, 4, EBTNodeResult::Succeeded);
				}
				FBTBuilder::AddTask(CompNode2, 6, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(7);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Abort: during service ceases relevant")

struct FAITest_BTAbortingDuringTaskServiceTick : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringTaskServiceTick()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, TEXT("Bool1"));

			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringTaskServiceTick, "System.AI.Behavior Trees.Abort: during task service tick")

struct FAITest_BTAbortingDuringTaskServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringTaskServiceBecomeRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringTaskServiceBecomeRelevant, "System.AI.Behavior Trees.Abort: during task service becomes relevant")

struct FAITest_BTAbortingDuringTaskServiceBecomeRelevantTaskTicking : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringTaskServiceBecomeRelevantTaskTicking()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringTaskServiceBecomeRelevantTaskTicking, "System.AI.Behavior Trees.Abort: during task service becomes relevant when task continue to tick for a longer time")

struct FAITest_BTAbortingDuringTaskServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTAbortingDuringTaskServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceLog(CompNode2, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, NAME_None, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 6, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(7);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortingDuringTaskServiceCeaseRelevant, "System.AI.Behavior Trees.Abort: during task service ceases relevant")

struct FAITest_BTSwitchingHigherPrioDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringServiceTick()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(CompNode3, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode3, 6, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringServiceTick, "System.AI.Behavior Trees.Switch: higher priority during service tick")

struct FAITest_BTRequestExecutionOnLatentAbortFinished : public FAITest_SimpleBT
{
	FAITest_BTRequestExecutionOnLatentAbortFinished()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset); // 0
		{
			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 3
				1 /* ExecHalfNumTicks */, FName() /* ExecKeyName */, 100 /* ExecLogStart */, 199 /* ExecLogFinish */,
				1 /* AbortHalfNumTicks */, FName() /* AbortKeyName */, 200 /* AbortLogStart */, 299 /* AbortLogFinish */);
			{
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(CompNode); // 1
				FBTBuilder::WithDecoratorDelayedAbort(CompNode, 1, false /* bOnlyOnce */); // 2
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded); // 4
			FBTBuilder::AddTask(CompNode, 5, EBTNodeResult::Succeeded); // 5
		}

		ExpectedResult.Add(100);
		ExpectedResult.Add(200);
		ExpectedResult.Add(299);
		ExpectedResult.Add(4);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRequestExecutionOnLatentAbortFinished, "System.AI.Behavior Trees.Abort: request on latent task finished")

struct FAITest_BTRequestExecutionOnTimerBasedLatentAbortFinished : public FAITest_SimpleBT
{
	FAITest_BTRequestExecutionOnTimerBasedLatentAbortFinished()
	{
		enum
		{
			FirstTaskStartExecute = 1,
			FirstTaskStartAbort,
			FirstTaskFinishAbort,
			FirstTaskFinished,
			SecondTaskExecute
		};

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& SelectorNode = FBTBuilder::AddSelector(CompNode);
			{
				constexpr float NumTicksBeforeAbortingTask = 3;

				// Add this decorator to prevent the test from running indefinitely if there is a regression on the behavior
				// this unit test is meant to validate 
				constexpr float NumTicksBeforeAbortingSelector = NumTicksBeforeAbortingTask + 10;
				FBTBuilder::WithDecorator<UBTDecorator_TimeLimit>(CompNode).TimeLimit = NumTicksBeforeAbortingSelector * FAITestHelpers::TickInterval;

				UTestBTTask_TimerBasedLatent& Task = FBTBuilder::AddTask<UTestBTTask_TimerBasedLatent>(SelectorNode);
				Task.LogIndexExecuteStart = FirstTaskStartExecute;
				Task.LogIndexAbortStart = FirstTaskStartAbort;
				Task.LogIndexAbortFinish = FirstTaskFinishAbort;
				Task.NumTicksAborting = 2;
				// Make sure to execute long enough to be aborted
				Task.NumTicksExecuting = 2 * NumTicksBeforeAbortingTask;
				{
					FBTBuilder::WithDecorator<UBTDecorator_TimeLimit>(SelectorNode).TimeLimit = NumTicksBeforeAbortingTask * FAITestHelpers::TickInterval;
				}

				FBTBuilder::AddTask(SelectorNode, SecondTaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(FirstTaskStartExecute);
		ExpectedResult.Add(FirstTaskStartAbort);
		ExpectedResult.Add(FirstTaskFinishAbort);
		ExpectedResult.Add(SecondTaskExecute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTRequestExecutionOnTimerBasedLatentAbortFinished, "System.AI.Behavior Trees.Abort: request on latent timer based task finished")

struct FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(CompNode3, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode3, 6, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevant, "System.AI.Behavior Trees.Switch: higher priority during service becomes relevant")

struct FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevantWithTickingTask : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevantWithTickingTask()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(CompNode3, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode3, 6, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringServiceBecomeRelevantWithTickingTask, "System.AI.Behavior Trees.Switch: higher priority during service becomes relevant with ticking task")

struct FAITest_BTSwitchingHigherPrioDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode);
			{
				UBTCompositeNode& CompNode4 = FBTBuilder::AddSelector(CompNode3);
				{
					FBTBuilder::WithServiceLog(CompNode4, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, NAME_None, TEXT("Bool1"));

					FBTBuilder::AddTask(CompNode4, 6, EBTNodeResult::Succeeded);
				}
				FBTBuilder::AddTask(CompNode3, 7, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(7);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Switch: higher priority during service ceases relevant")

struct FAITest_BTSwitchingHigherPrioDuringTaskServiceTick : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringTaskServiceTick()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, TEXT("Bool1"));
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringTaskServiceTick, "System.AI.Behavior Trees.Switch: higher priority during task service tick")

struct FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevant, "System.AI.Behavior Trees.Switch: higher priority during task service becomes relevant")

struct FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevantWithTickingTask : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevantWithTickingTask()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/, 222/*LogTickIndex*/);
			FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, TEXT("Bool1"));
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(222);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringTaskServiceBecomeRelevantWithTickingTask, "System.AI.Behavior Trees.Switch: higher priority during task service becomes relevant with ticking task")

struct FAITest_BTSwitchingHigherPrioDuringTaskServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTSwitchingHigherPrioDuringTaskServiceCeaseRelevant()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 4, EBTNodeResult::Succeeded);
			}
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::AddTask(CompNode3, 6, EBTNodeResult::Succeeded);
				FBTBuilder::WithTaskServiceLog(CompNode3, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 3/*TickIndex*/, NAME_None, false, NAME_None, TEXT("Bool1"));
				FBTBuilder::AddTask(CompNode3, 7, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(6);
		ExpectedResult.Add(3); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(7);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSwitchingHigherPrioDuringTaskServiceCeaseRelevant, "System.AI.Behavior Trees.Switch: higher priority during task service ceases relevant")

struct FAITest_BTAbortSelfFail : public FAITest_SimpleBT
{
	FAITest_BTAbortSelfFail()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self);

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortSelfFail, "System.AI.Behavior Trees.Abort: self failure")

struct FAITest_BTAbortSelfSuccess : public FAITest_SimpleBT
{
	FAITest_BTAbortSelfSuccess()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self);
				FBTBuilder::WithDecorator<UBTDecorator_ForceSuccess>(CompNode);

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortSelfSuccess, "System.AI.Behavior Trees.Abort: self success")

struct FAITest_BTAbortLowerPri : public FAITest_SimpleBT
{
	FAITest_BTAbortLowerPri()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
			}

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Failed);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(0);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortLowerPri, "System.AI.Behavior Trees.Abort: lower priority")

struct FAITest_BTAbortMerge1 : public FAITest_SimpleBT
{
	FAITest_BTAbortMerge1()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
				FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode);
			}

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self);

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 2, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortMerge1, "System.AI.Behavior Trees.Abort: merge ranges 1")

struct FAITest_BTAbortMerge2 : public FAITest_SimpleBT
{
	FAITest_BTAbortMerge2()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			FBTBuilder::AddTask(CompNode, 1, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
				FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode);
			}

			FBTBuilder::AddTask(CompNode, 2, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
			FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortMerge2, "System.AI.Behavior Trees.Abort: merge ranges 2")

struct FAITest_BTAbortMerge3 : public FAITest_SimpleBT
{
	FAITest_BTAbortMerge3()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode2);
				}

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
					FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode2);
				}
			}

			UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode3, 2, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
				}

				FBTBuilder::AddTaskFlagChange(CompNode3, true, EBTNodeResult::Failed);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortMerge3, "System.AI.Behavior Trees.Abort: merge ranges 3")

struct FAITest_BTAbortParallelInternal : public FAITest_SimpleBT
{
	FAITest_BTAbortParallelInternal()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::WaitForBackground);
			{
				FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Succeeded, 5);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 1, EBTNodeResult::Succeeded, 1);

					UBTCompositeNode& CompNode4 = FBTBuilder::AddSelector(CompNode3);
					{
						FBTBuilder::AddTask(CompNode4, 2, EBTNodeResult::Succeeded, 3);
						{
							FBTBuilder::WithDecoratorBlackboard(CompNode4, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
						}

						FBTBuilder::AddTask(CompNode4, 3, EBTNodeResult::Succeeded, 1);
					}

					FBTBuilder::AddTaskFlagChange(CompNode3, true, EBTNodeResult::Succeeded);
				}
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortParallelInternal, "System.AI.Behavior Trees.Abort: parallel internal")

struct FAITest_BTAbortParallelOut : public FAITest_SimpleBT
{
	FAITest_BTAbortParallelOut()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
			}

			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::WaitForBackground);
			{
				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Failed, 5);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 2, EBTNodeResult::Succeeded, 1);
					FBTBuilder::AddTaskFlagChange(CompNode3, true, EBTNodeResult::Succeeded);
					FBTBuilder::AddTask(CompNode3, 3, EBTNodeResult::Succeeded, 1);
				}
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
		ExpectedResult.Add(0);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortParallelOut, "System.AI.Behavior Trees.Abort: parallel out")

struct FAITest_BTAbortParallelOutAndBack : public FAITest_SimpleBT
{
	FAITest_BTAbortParallelOutAndBack()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
				FBTBuilder::WithDecorator<UTestBTDecorator_CantExecute>(CompNode);
			}

			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::WaitForBackground);
			{
				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Failed, 5);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 2, EBTNodeResult::Succeeded, 2);
					FBTBuilder::AddTaskFlagChange(CompNode3, true, EBTNodeResult::Succeeded);
					FBTBuilder::AddTask(CompNode3, 3, EBTNodeResult::Succeeded, 3);
				}
			}

			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortParallelOutAndBack, "System.AI.Behavior Trees.Abort: parallel out & back")

struct FAITest_BTAbortMultipleDelayed : public FAITest_SimpleBT
{
	FAITest_BTAbortMultipleDelayed()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorDelayedAbort(CompNode, 2, false);

				FBTBuilder::AddTaskLogFinish(CompNode2, 1, 11, EBTNodeResult::Succeeded, 4);
				FBTBuilder::AddTaskLogFinish(CompNode2, 2, 12, EBTNodeResult::Succeeded, 4);
			}
			
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortMultipleDelayed, "System.AI.Behavior Trees.Abort: multiple delayed requests")

struct FAITest_BTAbortToInactiveParallel : public FAITest_SimpleBT
{
	FAITest_BTAbortToInactiveParallel()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddParallel(CompNode, EBTParallelMode::WaitForBackground);
			{
				FBTBuilder::WithDecoratorDelayedAbort(CompNode, 5);

				FBTBuilder::AddTaskLogFinish(CompNode2, 1, 11, EBTNodeResult::Succeeded, 10);

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(CompNode2);
				{
					FBTBuilder::AddTask(CompNode3, 2, EBTNodeResult::Succeeded);
					{
						FBTBuilder::WithDecoratorBlackboard(CompNode3, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);
					}

					FBTBuilder::AddTaskLogFinish(CompNode3, 3, 13, EBTNodeResult::Succeeded, 8);
				}
			}

			UBTCompositeNode& CompNode4 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::AddTask(CompNode4, 4, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode4, true, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode4, 5, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(5);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortToInactiveParallel, "System.AI.Behavior Trees.Abort: observer in inactive parallel")

struct FAITest_BTAbortDuringLatentAbort : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringLatentAbort()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, 1, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool3"));
				}
			}

			UBTCompositeNode& Comp2Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));

				FBTBuilder::AddTaskLatentFlags(Comp2Node, EBTNodeResult::Succeeded,
					1, TEXT("Bool2"), 2, 3,
					1, TEXT("Bool1"), 4, 5);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp2Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool2"));
				}

				FBTBuilder::AddTask(Comp2Node, 6, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
		ExpectedResult.Add(5);
		ExpectedResult.Add(7);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringLatentAbort, "System.AI.Behavior Trees.Abort: during latent task abort (lower pri)")

struct FAITest_BTAbortDuringLatentAbort2 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringLatentAbort2()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTaskLatentFlags(Comp1Node, EBTNodeResult::Succeeded,
					1, TEXT("Bool1"), 0, 1,
					1, TEXT("Bool2"), 2, 3);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
				}

				FBTBuilder::AddTask(Comp1Node, 4, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool3"));
				}
			}

			FBTBuilder::AddTask(CompNode, 5, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringLatentAbort2, "System.AI.Behavior Trees.Abort: during latent task abort (self)")

struct FAITest_BTAbortDuringLatentAbort3 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringLatentAbort3()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool1")); // 1
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 4
				1, TEXT("Bool1"), 0, 1,
				1, TEXT("Bool2"), 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2")); // 3
			}

			FBTBuilder::AddTask(CompNode, 5, EBTNodeResult::Succeeded); // 5
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringLatentAbort3, "System.AI.Behavior Trees.Abort: during latent task abort (high pri)")

struct FAITest_BTAbortDuringLatentAbort4 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringLatentAbort4()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2")); // 1
				FBTBuilder::AddTask(Comp1Node, 8, EBTNodeResult::Succeeded); // 4
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool3")); // 3
				}
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 6
				1, TEXT("Bool1"), 0, 1,
				1, TEXT("Bool2"), 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1")); // 5
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 8
				1, TEXT("Bool3"), 4, 5,
				0, NAME_None, 6, 7);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool4")); // 7
			}
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(6);
		ExpectedResult.Add(7);
		ExpectedResult.Add(8);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringLatentAbort4, "System.AI.Behavior Trees.Abort: during latent task abort (high pri 2)")

struct FAITest_BTResumingDuringLatentAbort : public FAITest_SimpleBT
{
	FAITest_BTResumingDuringLatentAbort()
	{
		enum
		{
			LatentTaskStart = 0,
			LatentTaskFinish,
			LatentTaskAbortStart,
			LatentTaskAbortFinish,
			Task1Log,
			Task2Log,
		};


		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			FBTBuilder::AddTask(CompNode, Task1Log, EBTNodeResult::Succeeded); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool1")); // 1
			}

			FBTBuilder::AddTask(CompNode, Task2Log, EBTNodeResult::Succeeded); // 4
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool2")); // 3
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 5
				/*ExecuteHalfTicks*/2, TEXT("Bool1"), LatentTaskStart, LatentTaskFinish,
				/*AbortHalfTicks*/3, TEXT("Bool1"), LatentTaskAbortStart, LatentTaskAbortFinish, EBTTestChangeFlagBehavior::Toggle);
			{
				FBTBuilder::WithTaskServiceLog(CompNode, /*ActivationIndex*/-1, /*DeactivationIndex*/-1, /*TickIndex*/-1, /*TickBoolKeyName*/TEXT("Bool2")); // 6
			}
		}

		ExpectedResult.Add(LatentTaskStart/*0*/);
		ExpectedResult.Add(LatentTaskAbortStart/*2*/);
		ExpectedResult.Add(LatentTaskAbortFinish/*3*/);
		ExpectedResult.Add(Task2Log/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTResumingDuringLatentAbort, "System.AI.Behavior Trees.Resume: during latent task abort")

struct FAITest_BTAbortDuringInstantAbort : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringInstantAbort()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, 1, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool3"));
				}
			}

			UBTCompositeNode& Comp2Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));

				FBTBuilder::AddTaskLatentFlags(Comp2Node, EBTNodeResult::Succeeded,
					1, TEXT("Bool2"), 2, 3,
					0, TEXT("Bool1"), 4, 5);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp2Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool2"));
				}

				FBTBuilder::AddTask(Comp2Node, 6, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(2);
		ExpectedResult.Add(4);
		ExpectedResult.Add(5);
		ExpectedResult.Add(7);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringInstantAbort, "System.AI.Behavior Trees.Abort: during instant task abort (lower pri)")

struct FAITest_BTAbortDuringInstantAbort2 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringInstantAbort2()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTaskLatentFlags(Comp1Node, EBTNodeResult::Succeeded,
					1, TEXT("Bool1"), 0, 1, 
					0, TEXT("Bool2"), 2, 3);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1"));
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2"));
				}

				FBTBuilder::AddTask(Comp1Node, 4, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool3"));
				}
			}

			FBTBuilder::AddTask(CompNode, 5, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringInstantAbort2, "System.AI.Behavior Trees.Abort: during instant task abort (self)")

struct FAITest_BTAbortDuringInstantAbort3 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringInstantAbort3()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			FBTBuilder::AddTask(CompNode, 4, EBTNodeResult::Succeeded); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool1")); // 1
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 4
				1, TEXT("Bool1"), 0, 1,
				0, TEXT("Bool2"), 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool2")); // 3
			}

			FBTBuilder::AddTask(CompNode, 5, EBTNodeResult::Succeeded); // 5
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringInstantAbort3, "System.AI.Behavior Trees.Abort: during instant task abort (high pri)")

struct FAITest_BTAbortDuringInstantAbort4 : public FAITest_SimpleBT
{
	FAITest_BTAbortDuringInstantAbort4()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2")); // 1
				FBTBuilder::AddTask(Comp1Node, 8, EBTNodeResult::Succeeded); // 4
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool3")); // 3
				}
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 6
				1, TEXT("Bool1"), 0, 1,
				0, TEXT("Bool2"), 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1")); // 5
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 8
				1, TEXT("Bool3"), 4, 5,
				0, NAME_None, 6, 7);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool4")); // 7
			}
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(6);
		ExpectedResult.Add(7);
		ExpectedResult.Add(8);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortDuringInstantAbort4, "System.AI.Behavior Trees.Abort: during instant task abort (high pri 2)")

struct FAITest_BTAbortOnValueChangePass : public FAITest_SimpleBT
{
	FAITest_BTAbortOnValueChangePass()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::NotEqual, 10, EBTFlowAbortMode::Self, EBTBlackboardRestart::ValueChange, TEXT("Int"));

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskValueChange(CompNode2, 1, EBTNodeResult::Succeeded, TEXT("Int"));
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortOnValueChangePass, "System.AI.Behavior Trees.Abort: value change (pass)")

struct FAITest_BTAbortOnValueChangeFail : public FAITest_SimpleBT
{
	FAITest_BTAbortOnValueChangeFail()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::NotEqual, 10, EBTFlowAbortMode::Self, EBTBlackboardRestart::ValueChange, TEXT("Int"));

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskValueChange(CompNode2, 10, EBTNodeResult::Succeeded, TEXT("Int"));
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortOnValueChangeFail, "System.AI.Behavior Trees.Abort: value change (fail)")

struct FAITest_BTAbortOnValueChangeFailOther : public FAITest_SimpleBT
{
	FAITest_BTAbortOnValueChangeFailOther()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::NotEqual, 10, EBTFlowAbortMode::Self, EBTBlackboardRestart::ValueChange, TEXT("Int"));
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::None, TEXT("Bool1"));

				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded, TEXT("Bool1"));
				FBTBuilder::AddTaskValueChange(CompNode2, 1, EBTNodeResult::Succeeded, TEXT("Int"));
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Failed);
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTAbortOnValueChangeFailOther, "System.AI.Behavior Trees.Abort: value change (other failed)")

struct FAITest_BTLowPriObserverInLoop : public FAITest_SimpleBT
{
	FAITest_BTLowPriObserverInLoop()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 1, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithDecoratorLoop(CompNode);

				FBTBuilder::AddTaskLatentFlags(CompNode2, EBTNodeResult::Succeeded, 1, TEXT("Bool2"), 2, 3);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool1"));
				}

				UBTCompositeNode& CompNode4 = FBTBuilder::AddSequence(CompNode2);
				{
					FBTBuilder::AddTask(CompNode4, 4, EBTNodeResult::Succeeded);
					FBTBuilder::AddTaskFlagChange(CompNode4, true, EBTNodeResult::Failed, TEXT("Bool1"));
				}

				FBTBuilder::AddTask(CompNode2, 5, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool2"));
				}
			}

			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(4);
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTLowPriObserverInLoop, "System.AI.Behavior Trees.Other: low priority observer in looped branch")

struct FAITest_BTLoopUntilSuccess : public FAITest_SimpleBT
{
	FAITest_BTLoopUntilSuccess()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			// We first set the bool key to false so task number 2 doesn't execute during the first loop
			FBTBuilder::AddTaskFlagChange(CompNode, false, EBTNodeResult::Failed);

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				// Loop the selector until it succeeds
				FBTBuilder::WithDecoratorLoopUntil(CompNode, EBTNodeResult::Succeeded);

				// Execute a failed task in the selector so we continue and can verify the looping
				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Failed);

				// Conditionally execute a succeeding task
				FBTBuilder::AddTask(CompNode2, 2, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::None, TEXT("Bool1"));
				}

				// Set the bool key to true so task number 2 will now execute and make the selector succeed
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Failed);
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTLoopUntilSuccess, "System.AI.Behavior Trees.LoopUntil: Loop until success")

struct FAITest_BTSubtreeSimple : public FAITest_SimpleBT
{
	FAITest_BTSubtreeSimple()
	{
		UBehaviorTree* ChildAsset1 = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset1)
		{
			AddAutoDestroyObject(*ChildAsset1);
			UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*ChildAsset1);
			{
				FBTBuilder::AddTask(CompNode, 10, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode, 11, EBTNodeResult::Succeeded);
			}
		}

		UBehaviorTree* ChildAsset2 = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset2)
		{
			AddAutoDestroyObject(*ChildAsset2);
			UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*ChildAsset2);
			{
				FBTBuilder::AddTask(CompNode, 20, EBTNodeResult::Failed);
				FBTBuilder::AddTask(CompNode, 21, EBTNodeResult::Succeeded);
			}
		}

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Failed);
				FBTBuilder::AddTaskSubtree(CompNode2, ChildAsset2);
				FBTBuilder::AddTask(CompNode2, 1, EBTNodeResult::Failed);
			}

			UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::AddTask(CompNode3, 2, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskSubtree(CompNode3, ChildAsset1);
				FBTBuilder::AddTask(CompNode3, 3, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(20);
		ExpectedResult.Add(1);
		ExpectedResult.Add(2);
		ExpectedResult.Add(10);
		ExpectedResult.Add(11);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSubtreeSimple, "System.AI.Behavior Trees.Subtree: simple")

struct FAITest_BTSubtreeRequestInDeactivatedBranch : public FAITest_SimpleBT
{
	FAITest_BTSubtreeRequestInDeactivatedBranch()
	{
		UBehaviorTree* ChildAsset1 = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset1)
		{
			AddAutoDestroyObject(*ChildAsset1);
			UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*ChildAsset1);
			{
				FBTBuilder::AddTask(CompNode, 10, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool2"));
				}
				FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::InProgress, TEXT("Bool1"), TEXT("Bool2"), true);
			}
		}

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::AddTask(CompNode2, 0, EBTNodeResult::Succeeded);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode2, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));
				}
				FBTBuilder::AddTaskSubtree(CompNode2, ChildAsset1);
			}
		}

		ExpectedResult.Add(0);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSubtreeRequestInDeactivatedBranch, "System.AI.Behavior Trees.Subtree: deactivated branch")

struct FAITest_BTSubtreeAbortOut : public FAITest_SimpleBT
{
	FAITest_BTSubtreeAbortOut()
	{
		UBehaviorTree* ChildAsset = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset)
		{
			AddAutoDestroyObject(*ChildAsset);
			UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*ChildAsset);
			{
				FBTBuilder::AddTask(CompNode, 10, EBTNodeResult::Succeeded);
				FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::Succeeded);
				FBTBuilder::AddTask(CompNode, 11, EBTNodeResult::Succeeded);
			}
		}

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded); 
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority);			
			}

			FBTBuilder::AddTaskSubtree(CompNode, ChildAsset);
		}

		ExpectedResult.Add(10);
		ExpectedResult.Add(0);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSubtreeAbortOut, "System.AI.Behavior Trees.Subtree: abort out")

/* This unittest come from UDN case 00320135 */
struct FAITest_BTSubtreeAbortOutToLowerPrio : public FAITest_SimpleBT
{
	FAITest_BTSubtreeAbortOutToLowerPrio()
	{
		UBehaviorTree* ChildAsset = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset)
		{
			AddAutoDestroyObject(*ChildAsset);
			UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*ChildAsset);
			{
				UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2"), 3/*LogIndexBecomeRelevant*/, 4/*LogIndexCeaseRelevant*/);
					FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded, TEXT("Bool3"));
				}

				UBTCompositeNode& CompNode3 = FBTBuilder::AddSequence(CompNode);
				{
					FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool3"), 5/*LogIndexBecomeRelevant*/, 6/*LogIndexCeaseRelevant*/);
					FBTBuilder::AddTask(CompNode3, 12, EBTNodeResult::Succeeded, 10/*ExecutionTicks*/);
					FBTBuilder::WithTaskServiceLog(CompNode3, 13/*ActivationIndex*/, 14/*DeactivationIndex*/, 15/*TickIndex*/, TEXT("Bool1"));
				}
			}
		}

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTaskFlagChange(CompNode, true, EBTNodeResult::Failed, TEXT("Bool3"));
			FBTBuilder::AddTaskSubtree(CompNode, ChildAsset);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"), 1/*LogIndexBecomeRelevant*/, 2/*LogIndexCeaseRelevant*/);
			}

			UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
			{
				FBTBuilder::AddTaskFlagChange(CompNode2, false, EBTNodeResult::Succeeded, TEXT("Bool3"));
				FBTBuilder::AddTaskFlagChange(CompNode2, true, EBTNodeResult::Succeeded, TEXT("Bool2"));
				FBTBuilder::AddTaskFlagChange(CompNode2, false, EBTNodeResult::Succeeded, TEXT("Bool1"));
			}
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(3);
		ExpectedResult.Add(13);
		ExpectedResult.Add(15);
		ExpectedResult.Add(12);
		ExpectedResult.Add(15); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(14);
		ExpectedResult.Add(4);
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(2);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSubtreeAbortOutToLowerPrio, "System.AI.Behavior Trees.Subtree: abort out lower prio")

/* This unittest come from UDN case 00484061 */
struct FAITest_BTSubtreeAbortOutToLowerPrio2 : public FAITest_SimpleBT
{
	FAITest_BTSubtreeAbortOutToLowerPrio2()
	{
		enum
		{
			SubtreeTaskExecute = 1,
			SubtreeTaskTick,
			BBDecoratorBecomeRelevant,
			BBDecoratorCeaseRelevant,
			MainTaskExecute,
			MainTaskTick
		};

		// SubTree
		UBehaviorTree* ChildAsset = &FBTBuilder::CreateBehaviorTree(*BTAsset);
		if (ChildAsset)
		{
			AddAutoDestroyObject(*ChildAsset);
			UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*ChildAsset);
			{
				// SubtreeTask
				FBTBuilder::AddTask(CompNode, SubtreeTaskExecute, EBTNodeResult::Failed, 1, SubtreeTaskTick);
			}
		}

		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			// SubTreeTask
			FBTBuilder::AddTaskSubtree(CompNode, ChildAsset);

			// BBDecorator
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"), BBDecoratorBecomeRelevant, BBDecoratorCeaseRelevant);

			// MainTask
			FBTBuilder::AddTask(CompNode, MainTaskExecute, EBTNodeResult::Succeeded, 1, MainTaskTick);
		}

		ExpectedResult.Add(SubtreeTaskExecute/*1*/);
		ExpectedResult.Add(SubtreeTaskTick/*2*/);
		ExpectedResult.Add(SubtreeTaskTick/*2*/);
		ExpectedResult.Add(BBDecoratorBecomeRelevant/*3*/);
		ExpectedResult.Add(MainTaskExecute/*5*/);
		ExpectedResult.Add(MainTaskTick/*6*/);
		ExpectedResult.Add(MainTaskTick/*6*/);
		ExpectedResult.Add(BBDecoratorCeaseRelevant/*4*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTSubtreeAbortOutToLowerPrio2, "System.AI.Behavior Trees.Subtree: abort out lower prio 2")

struct FAITest_BTServiceInstantTask : public FAITest_SimpleBT
{
	FAITest_BTServiceInstantTask()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 4/*TickIndex*/);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(4);
		ExpectedResult.Add(0);
		ExpectedResult.Add(4); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTServiceInstantTask, "System.AI.Behavior Trees.Service: instant task")

struct FAITest_BTServiceLatentTask : public FAITest_SimpleBT
{
	FAITest_BTServiceLatentTask()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSequence(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded, 2);
			{
				FBTBuilder::WithTaskServiceLog(CompNode, 1/*ActivationIndex*/, 2/*DeactivationIndex*/, 4/*TickIndex*/);
			}

			FBTBuilder::AddTask(CompNode, 3, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(1);
		ExpectedResult.Add(4);
		ExpectedResult.Add(0);
		ExpectedResult.Add(4);
		ExpectedResult.Add(4);
		ExpectedResult.Add(4);// Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(2);
		ExpectedResult.Add(3);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTServiceLatentTask, "System.AI.Behavior Trees.Service: latent task")

struct FAITest_BTServiceAbortingTask : public FAITest_SimpleBT
{
	FAITest_BTServiceAbortingTask()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset);
		{
			FBTBuilder::AddTask(CompNode, 0, EBTNodeResult::Succeeded);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::LowerPriority, TEXT("Bool1"));
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, 1, TEXT("Bool1"), 1, 2, 0, NAME_None, 3, 4);
			{
				FBTBuilder::WithTaskServiceLog(CompNode, 5/*ActivationIndex*/, 6/*DeactivationIndex*/, 8/*TickIndex*/);
			}

			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(5);
		ExpectedResult.Add(8);
		ExpectedResult.Add(1);
		ExpectedResult.Add(8);
		ExpectedResult.Add(8); // Extra service tick because aux nodes are always ticked before any pending requests are processed
		ExpectedResult.Add(3);
		ExpectedResult.Add(4);
		ExpectedResult.Add(6);
		ExpectedResult.Add(0);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTServiceAbortingTask, "System.AI.Behavior Trees.Service: abort task")

struct FAITest_BTPostponeDuringSearch : public FAITest_SimpleBT
{
	FAITest_BTPostponeDuringSearch()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			FBTBuilder::AddTask(CompNode, 6, EBTNodeResult::Succeeded); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2")); // 1
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 4
				1, TEXT("Bool1"), 0, 1, 
				0, NAME_None, 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1")); // 3
			}

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode); // 7
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool3")); // 5
				FBTBuilder::WithServiceLog(Comp1Node, INDEX_NONE, INDEX_NONE, 4, TEXT("Bool2"), true); // 6
				FBTBuilder::AddTask(Comp1Node, 5, EBTNodeResult::Succeeded); // 8
			}

			FBTBuilder::AddTask(CompNode, 7, EBTNodeResult::Succeeded); // 9
		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(4);
		ExpectedResult.Add(1);
		ExpectedResult.Add(6);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTPostponeDuringSearch, "System.AI.Behavior Trees.Service: Postpone during search (high pri)")

struct FAITest_BTPostponeDuringSearch2 : public FAITest_SimpleBT
{
	FAITest_BTPostponeDuringSearch2()
	{
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(*BTAsset); // 0
		{
			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode); // 2
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool2")); // 1
				FBTBuilder::AddTask(Comp1Node, 9, EBTNodeResult::Succeeded); // 4
				{
					FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::Set, EBTFlowAbortMode::Both, TEXT("Bool3")); // 3
				}
			}

			FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 6
				1, TEXT("Bool1"), 0, 1,
				0, NAME_None, 2, 3);
			{
				FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Self, TEXT("Bool1")); // 5
			}

			UBTCompositeNode& Comp2Node = FBTBuilder::AddSelector(CompNode); // 8
			{
				FBTBuilder::WithServiceLog(Comp2Node, INDEX_NONE, INDEX_NONE, 4, TEXT("Bool2"), true); // 7
				FBTBuilder::AddTaskLatentFlags(CompNode, EBTNodeResult::Succeeded, // 9
					1, TEXT("Bool3"), 5, 6,
					0, NAME_None, 7, 8);
			}

		}

		ExpectedResult.Add(0);
		ExpectedResult.Add(4);
		ExpectedResult.Add(1);
		ExpectedResult.Add(4);
		ExpectedResult.Add(5);
		ExpectedResult.Add(7);
		ExpectedResult.Add(8);
		ExpectedResult.Add(9);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTPostponeDuringSearch2, "System.AI.Behavior Trees.Service: Postpone during search (lower pri)")

// This test is temporarily turned off until we enable CVarApplyAuxNodesFromFailedSearches again
//struct FAITest_BTTestOberserverAbortLowerPriorityTwoDeep: public FAITest_SimpleBT
//{
//	FAITest_BTTestOberserverAbortLowerPriorityTwoDeep()
//	{
//		enum
//		{
//			MainTaskExecute = 1,
//			InteruptingTaskExecute,
//		};
//
//		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
//		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
//
//		{
//			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
//			FBTBuilder::WithDecoratorBlackboard(CompNode, EArithmeticKeyOperation::Equal, 1, EBTFlowAbortMode::LowerPriority, EBTBlackboardRestart::ValueChange, TEXT("Int"));
//
//			UBTCompositeNode& Comp2Node = FBTBuilder::AddSelector(Comp1Node);
//			FBTBuilder::WithDecoratorBlackboard(Comp1Node, EArithmeticKeyOperation::Equal, 1, EBTFlowAbortMode::LowerPriority, EBTBlackboardRestart::ValueChange, TEXT("Int2"));
//
//			FBTBuilder::AddTask(Comp2Node, InteruptingTaskExecute, EBTNodeResult::Succeeded);
//		}
//
//		{
//			FBTBuilder::AddTaskValuesChangedWithLogs(/*ParentNode*/ CompNode, /*LogIndex*/ MainTaskExecute, /*NodeResult*/ EBTNodeResult::Succeeded, /*Value1*/1, /*Value2*/ 1, /*IntKeyName2*/ TEXT("Int"), /* IntKeyName2 */ TEXT("Int2"), /*ExecutionTicks1*/ 5, /*ExecutionTicks2*/ 5);
//		}
//
//		ExpectedResult.Add(MainTaskExecute);
//		ExpectedResult.Add(InteruptingTaskExecute);
//	}
//};
//IMPLEMENT_AI_LATENT_TEST(FAITest_BTTestOberserverAbortLowerPriorityTwoDeep, "System.AI.Behavior Trees.OberserverAbortLowerPriorityTwoDeep: Two nodes in same branch with observer aborts lower priority")

struct FAITest_BTTestTickAuxNodesDeactivateNodes : public FAITest_SimpleBT
{
	FAITest_BTTestTickAuxNodesDeactivateNodes()
	{
		enum
		{
			MainTaskExecute = 1,
			StartAbortMainTask,
			SecondaryTask,
			IncorrectTaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		FBTBuilder::WithServiceLog(/*ParentNode*/ RootNode, /*ActivationIndex*/ INDEX_NONE, /*DeactivationIndex*/ INDEX_NONE, /*TickIndex*/ INDEX_NONE, /*TickBoolKeyName*/ TEXT("Bool1"), /*bCallTickOnSearchStart*/ false, /*BecomeRelevantBoolKeyName*/ NAME_None, /*BecomeRelevantBoolKeyName*/ NAME_None, /*bToggleValue*/ false, /*TicksDelaySetKeyNameTick*/ 4);
		FBTBuilder::WithServiceLog(/*ParentNode*/ RootNode, /*ActivationIndex*/ INDEX_NONE, /*DeactivationIndex*/ INDEX_NONE, /*TickIndex*/ INDEX_NONE, /*TickBoolKeyName*/ TEXT("Bool3"), /*bCallTickOnSearchStart*/ false, /*BecomeRelevantBoolKeyName*/ NAME_None, /*BecomeRelevantBoolKeyName*/ NAME_None, /*bToggleValue*/ false, /*TicksDelaySetKeyNameTick*/ 2);

		{
			UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
			FBTBuilder::WithDecoratorBlackboard(/*ParentNode*/ RootNode, /*Condition*/ EBasicKeyOperation::NotSet, /*Observer*/ EBTFlowAbortMode::LowerPriority, /*BoolKeyName*/  TEXT("Bool1"));
			{
				UBTCompositeNode& CompNode2 = FBTBuilder::AddSelector(CompNode);
				FBTBuilder::WithDecoratorBlackboard(/*ParentNode*/ RootNode, /*Condition*/ EBasicKeyOperation::Set, /*Observer*/ EBTFlowAbortMode::LowerPriority, /*BoolKeyName*/ TEXT("Bool2"));
				{
					FBTBuilder::AddTaskLogFinish(/*ParentNode*/ CompNode2, /*LogIndex*/ IncorrectTaskExecute, /*FinishIndex*/ INDEX_NONE, EBTNodeResult::Succeeded, /*ExecutionTick*/ 0);
				}
			}
		}

		{
			UBTCompositeNode& CompNode3 = FBTBuilder::AddSelector(RootNode);
			FBTBuilder::WithDecoratorBlackboard(/*ParentNode*/ RootNode, /*Condition*/ EBasicKeyOperation::Set, /*Observer*/ EBTFlowAbortMode::LowerPriority, /*BoolKeyName*/ TEXT("Bool3"));

			FBTBuilder::AddTaskLogFinish(/*ParentNode*/ CompNode3, /*LogIndex*/ SecondaryTask, /*FinishIndex*/ INDEX_NONE, EBTNodeResult::Succeeded, /*ExecutionTick*/ 4);
		}

		{
			FBTBuilder::AddTaskLatentFlags(RootNode, EBTNodeResult::Succeeded, 3, FName(), MainTaskExecute, INDEX_NONE, 3, NAME_None, 2, INDEX_NONE);
		}

		ExpectedResult.Add(MainTaskExecute);
		ExpectedResult.Add(StartAbortMainTask);
		ExpectedResult.Add(SecondaryTask);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTTestTickAuxNodesDeactivateNodes, "System.AI.Behavior Trees.Ticking aux nodes deactivates nodes")

struct FAITest_BTTestAbortNestedSimpleParallel : public FAITest_SimpleBT
{
	FAITest_BTTestAbortNestedSimpleParallel()
	{
		enum
		{
			ChildFirstTaskExecute = 1,
			ChildSecondTaskExecute,
			ChildThirdTaskExecute
		};

		UBehaviorTree* ChildBTAsset = &FBTBuilder::CreateBehaviorTree();
		AddAutoDestroyObject(*ChildBTAsset);

		// Setup subtree
		{
			UBTCompositeNode& RootNode = FBTBuilder::AddSequence(*ChildBTAsset);
			{
				UBTCompositeNode& CompNode = FBTBuilder::AddParallel(RootNode, EBTParallelMode::AbortBackground);
				{
					FBTBuilder::AddTaskLogFinish(/*ParentNode*/ CompNode, /*LogIndex*/ ChildFirstTaskExecute, /*FinishIndex*/ INDEX_NONE, EBTNodeResult::Succeeded, /*ExecutionTick*/ 500);
					FBTBuilder::WithDecorator<UBTDecorator_Cooldown>(CompNode);

					UBTCompositeNode& CompNode2 = FBTBuilder::AddSequence(CompNode);
					{
						FBTBuilder::AddTaskLogFinish(/*ParentNode*/ CompNode2, /*LogIndex*/ ChildSecondTaskExecute, /*FinishIndex*/ INDEX_NONE, EBTNodeResult::Succeeded, /*ExecutionTick*/ 5);
						FBTBuilder::WithDecorator<UBTDecorator_Cooldown>(CompNode2);

						FBTBuilder::AddTaskLogFinish(/*ParentNode*/ CompNode2, /*LogIndex*/ ChildThirdTaskExecute, /*FinishIndex*/ INDEX_NONE, EBTNodeResult::Succeeded, /*ExecutionTick*/ 5);
					}
				}
			}
		}

		// Setup main tree
		{
			UBTCompositeNode& RootNode = FBTBuilder::AddSequence(*BTAsset);
			{
				constexpr float NumTicks = 4.f;
				UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
				FBTBuilder::WithDecorator<UBTDecorator_TimeLimit>(RootNode).TimeLimit = NumTicks * FAITestHelpers::TickInterval;
				{
					FBTBuilder::AddTask<UTestBTTask_RunBehavior>(CompNode).SetSubtreeAsset(ChildBTAsset);
				}
			}
		}

		ExpectedResult.Add(ChildFirstTaskExecute);
		ExpectedResult.Add(ChildSecondTaskExecute);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTTestAbortNestedSimpleParallel, "System.AI.Behavior Trees.Abort: nested simple parallel")

//////////////////////////////////////////////////////////////////////////////////////////////////
// Stop tree tests
//////////////////////////////////////////////////////////////////////////////////////////////////
struct FAITest_BTStopDuringTaskExecute : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskExecute()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			UninitTaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskExecute, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringExecute, EBTTestStopAction::StopTree);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskExecute/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskExecute, "System.AI.Behavior Trees.Stop: during task execute")

struct FAITest_BTStopDuringTaskTick : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			UninitTaskTick,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskTick, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringTick, EBTTestStopAction::StopTree);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskTick/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskTick, "System.AI.Behavior Trees.Stop: during task tick")

struct FAITest_BTStopDuringTaskAbort : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskAbort()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskAborting,
			TaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskAborting, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringAbort, EBTTestStopAction::StopTree);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*5*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskAborting/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskAbort, "System.AI.Behavior Trees.Stop: during task abort")

struct FAITest_BTStopDuringTaskFinished : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskFinished()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskFinished,
			TaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskFinished, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringFinish, EBTTestStopAction::StopTree);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskFinished/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskFinished, "System.AI.Behavior Trees.Stop: during task finished")

struct FAITest_BTStopDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTStopDuringServiceTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceTick,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Servier
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitServiceTick
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::StopTree);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringServiceTick, "System.AI.Behavior Trees.Stop: during service tick")

struct FAITest_BTStopDuringServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTStopDuringServiceBecomeRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::StopTree);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(UninitServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringServiceBecomeRelevant, "System.AI.Behavior Trees.Stop: during service become relevant")

struct FAITest_BTStopDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTStopDuringServiceCeaseRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::StopTree);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
			}
			// Task2
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(UninitServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Stop: during service cease relevant")

struct FAITest_BTStopDuringTaskServiceTick : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskServiceTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceTick,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::StopTree);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskServiceTick, "System.AI.Behavior Trees.Stop: during task service tick")

struct FAITest_BTStopDuringTaskServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskServiceBecomeRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::StopTree);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskServiceBecomeRelevant, "System.AI.Behavior Trees.Stop: during task service become relevant")

struct FAITest_BTStopDuringTaskServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTStopDuringTaskServiceCeaseRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::StopTree);

				FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTStopDuringTaskServiceCeaseRelevant, "System.AI.Behavior Trees.Stop: during task service cease relevant")

//////////////////////////////////////////////////////////////////////////////////////////////////
// Uninitialize tests
//////////////////////////////////////////////////////////////////////////////////////////////////
struct FAITest_BTUninitializeDuringTaskExecute : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskExecute()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    UninitTaskExecute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskExecute, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringExecute, EBTTestStopAction::UnInitialize);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskExecute/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskExecute, "System.AI.Behavior Trees.Uninitialize: during task execute")

struct FAITest_BTUninitializeDuringTaskTick : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskTick()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    UninitTaskTick,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskTick, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringTick, EBTTestStopAction::UnInitialize);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskTick/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskTick, "System.AI.Behavior Trees.Uninitialize: during task tick")

struct FAITest_BTUninitializeDuringTaskAbort : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskAbort()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitTaskAborting,
			TaskExecute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskAborting, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringAbort, EBTTestStopAction::UnInitialize);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*5*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskAborting/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskAbort, "System.AI.Behavior Trees.Uninitialize: during task abort")

struct FAITest_BTUninitializeDuringTaskFinished : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskFinished()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitTaskFinished,
			TaskExecute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskFinished, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringFinish, EBTTestStopAction::UnInitialize);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskFinished/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskFinished, "System.AI.Behavior Trees.Uninitialize: during task finished")

struct FAITest_BTUninitializeDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringServiceTick()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitServiceTick,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Servier
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitServiceTick
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::UnInitialize);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringServiceTick, "System.AI.Behavior Trees.Uninitialize: during service tick")

struct FAITest_BTUninitializeDuringServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringServiceBecomeRelevant()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::UnInitialize);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(UninitServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringServiceBecomeRelevant, "System.AI.Behavior Trees.Uninitialize: during service become relevant")

struct FAITest_BTUninitializeDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringServiceCeaseRelevant()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::UnInitialize);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
			}
			// Task2
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(UninitServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Uninitialize: during service cease relevant")

struct FAITest_BTUninitializeDuringTaskServiceTick : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskServiceTick()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitTaskServiceTick,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::UnInitialize);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskServiceTick, "System.AI.Behavior Trees.Uninitialize: during task service tick")

struct FAITest_BTUninitializeDuringTaskServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTUninitializeDuringTaskServiceBecomeRelevant()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitTaskServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::UnInitialize);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskServiceBecomeRelevant, "System.AI.Behavior Trees.Uninitialize: during task service become relevant")

struct FAITest_BTUninitializeDuringTaskServiceCeaseRelevant : public FAITest_SimpleBT
{	FAITest_BTUninitializeDuringTaskServiceCeaseRelevant()
	{
	    enum
	    {
		    ServiceActivate = 1,
		    ServiceDeactivate,
		    ServiceTick,
		    SubServiceActivate,
		    SubServiceDeactivate,
		    SubServiceTick,
		    UninitTaskServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
	    };

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::UnInitialize);

				FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTUninitializeDuringTaskServiceCeaseRelevant, "System.AI.Behavior Trees.Uninitialize: during task service cease relevant")

//////////////////////////////////////////////////////////////////////////////////////////////////
// Cleanup tests
//////////////////////////////////////////////////////////////////////////////////////////////////
struct FAITest_BTCleanupDuringTaskExecute : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskExecute()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			UninitTaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskExecute, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringExecute, EBTTestStopAction::Cleanup);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskExecute/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskExecute, "System.AI.Behavior Trees.Cleanup: during task execute")

struct FAITest_BTCleanupDuringTaskTick : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			UninitTaskTick,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskTick, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringTick, EBTTestStopAction::Cleanup);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(UninitTaskTick/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskTick, "System.AI.Behavior Trees.Cleanup: during task tick")

struct FAITest_BTCleanupDuringTaskAbort : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskAbort()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskAborting,
			TaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskAborting, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringAbort, EBTTestStopAction::Cleanup);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*5*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskAborting/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskAbort, "System.AI.Behavior Trees.Cleanup: during task abort")

struct FAITest_BTCleanupDuringTaskFinished : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskFinished()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskFinished,
			TaskExecute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				// UninitTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, UninitTaskFinished, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringFinish, EBTTestStopAction::Cleanup);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskFinished/*7*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskFinished, "System.AI.Behavior Trees.Cleanup: during task finished")

struct FAITest_BTCleanupDuringServiceTick : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringServiceTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceTick,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Servier
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitServiceTick
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::Cleanup);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringServiceTick, "System.AI.Behavior Trees.Cleanup: during service tick")

struct FAITest_BTCleanupDuringServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringServiceBecomeRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::Cleanup);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));

				// Task2
				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(UninitServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringServiceBecomeRelevant, "System.AI.Behavior Trees.Cleanup: during service become relevant")

struct FAITest_BTCleanupDuringServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringServiceCeaseRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				// SubService
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));
				// UninitService
				FBTBuilder::WithServiceBTStopAction(Comp1Node, UninitServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::Cleanup);

				// Task
				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
			}
			// Task2
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(UninitServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringServiceCeaseRelevant, "System.AI.Behavior Trees.Cleanup: during service cease relevant")

struct FAITest_BTCleanupDuringTaskServiceTick : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskServiceTick()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceTick,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceTick, EBTTestServiceStopTiming::DuringTick, EBTTestStopAction::Cleanup);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceTick/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskServiceTick, "System.AI.Behavior Trees.Cleanup: during task service tick")

struct FAITest_BTCleanupDuringTaskServiceBecomeRelevant : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskServiceBecomeRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceBecomeRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceBecomeRelevant, EBTTestServiceStopTiming::DuringBecomeRelevant, EBTTestStopAction::Cleanup);

				FBTBuilder::AddTask(Comp1Node, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceBecomeRelevant/*7*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskServiceBecomeRelevant, "System.AI.Behavior Trees.Cleanup: during task service become relevant")

struct FAITest_BTCleanupDuringTaskServiceCeaseRelevant : public FAITest_SimpleBT
{
	FAITest_BTCleanupDuringTaskServiceCeaseRelevant()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			SubServiceActivate,
			SubServiceDeactivate,
			SubServiceTick,
			UninitTaskServiceCeaseRelevant,
			TaskExecute,
			Task2Execute,
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			{
				FBTBuilder::WithServiceLog(Comp1Node, SubServiceActivate, SubServiceDeactivate, SubServiceTick, TEXT("Bool1"));

				FBTBuilder::AddTask(Comp1Node, TaskExecute, EBTNodeResult::Succeeded);
				FBTBuilder::WithDecoratorBlackboard(Comp1Node, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
				FBTBuilder::WithTaskServiceBTStopAction(Comp1Node, UninitTaskServiceCeaseRelevant, EBTTestServiceStopTiming::DuringCeaseRelevant, EBTTestStopAction::Cleanup);

				FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
			}
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(SubServiceActivate/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(TaskExecute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(SubServiceTick/*6*/);
		ExpectedResult.Add(UninitTaskServiceCeaseRelevant/*7*/);
		ExpectedResult.Add(SubServiceDeactivate/*5*/);
		ExpectedResult.Add(Task2Execute/*9*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCleanupDuringTaskServiceCeaseRelevant, "System.AI.Behavior Trees.Cleanup: during task service cease relevant")

//////////////////////////////////////////////////////////////////////////////////////////////////
// Restart tests
//////////////////////////////////////////////////////////////////////////////////////////////////
struct FAITest_BTDefaultRestartDuringTaskExecute : public FAITest_SimpleBT
{
	FAITest_BTDefaultRestartDuringTaskExecute()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			RestartTaskTaskExecute,
			ServiceActivate2,
			ServiceDeactivate2,
			ServiceTick2,
			Task2Execute
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
			{
				FBTBuilder::WithServiceLog(Comp1Node, ServiceActivate2, ServiceDeactivate2, ServiceTick2, /*TickBoolKeyName*/NAME_None, /*bCallOnTickSearch*/false, /*BecomeRelevantBoolKeyName*/TEXT("Bool1"));

				// RestartTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, RestartTaskTaskExecute, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringExecute, EBTTestStopAction::Restart_ForceReevaluateRootNode);
			}
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceActivate2/*5*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(ServiceTick2/*7*/);
		ExpectedResult.Add(RestartTaskTaskExecute/*4*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(ServiceTick2/*7*/);
		ExpectedResult.Add(ServiceDeactivate2/*6*/);
		ExpectedResult.Add(Task2Execute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTDefaultRestartDuringTaskExecute, "System.AI.Behavior Trees.Restart: default during task execute")

struct FAITest_BTCompleteRestartDuringTaskExecute : public FAITest_SimpleBT
{
	FAITest_BTCompleteRestartDuringTaskExecute()
	{
		enum
		{
			ServiceActivate = 1,
			ServiceDeactivate,
			ServiceTick,
			RestartTaskTaskExecute,
			ServiceActivate2,
			ServiceDeactivate2,
			ServiceTick2,
			Task2Execute
		};

		UBTCompositeNode& RootNode = FBTBuilder::AddSelector(*BTAsset);
		UBTCompositeNode& CompNode = FBTBuilder::AddSelector(RootNode);
		{
			// Service
			FBTBuilder::WithServiceLog(CompNode, ServiceActivate, ServiceDeactivate, ServiceTick);

			UBTCompositeNode& Comp1Node = FBTBuilder::AddSelector(CompNode);
			FBTBuilder::WithDecoratorBlackboard(CompNode, EBasicKeyOperation::NotSet, EBTFlowAbortMode::Both, TEXT("Bool1"));
			{
				FBTBuilder::WithServiceLog(Comp1Node, ServiceActivate2, ServiceDeactivate2, ServiceTick2, /*TickBoolKeyName*/NAME_None, /*bCallOnTickSearch*/false, /*BecomeRelevantBoolKeyName*/TEXT("Bool1"));

				// RestartTask
				FBTBuilder::AddTaskBTStopAction(Comp1Node, RestartTaskTaskExecute, EBTNodeResult::Succeeded, EBTTestTaskStopTiming::DuringExecute, EBTTestStopAction::Restart_Complete);
			}
			FBTBuilder::AddTask(CompNode, Task2Execute, EBTNodeResult::Succeeded);
		}

		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceActivate2/*5*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(ServiceTick2/*7*/);
		ExpectedResult.Add(RestartTaskTaskExecute/*4*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
		ExpectedResult.Add(ServiceDeactivate2/*6*/);
		ExpectedResult.Add(ServiceActivate/*1*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(Task2Execute/*8*/);
		ExpectedResult.Add(ServiceTick/*3*/);
		ExpectedResult.Add(ServiceDeactivate/*2*/);
	}
};
IMPLEMENT_AI_LATENT_TEST(FAITest_BTCompleteRestartDuringTaskExecute, "System.AI.Behavior Trees.Restart: complete during task execute")

#undef LOCTEXT_NAMESPACE