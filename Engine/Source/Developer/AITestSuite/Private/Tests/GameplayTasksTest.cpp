// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GameplayTask.h"
#include "AITestsCommon.h"
#include "MockGameplayTasks.h"

#define LOCTEXT_NAMESPACE "AITestSuite_GameplayTasksTest"

typedef FAITest_SimpleComponentBasedTest<UMockGameplayTasksComponent> FAITest_GameplayTasksTest;

static const FGameplayResourceSet::FResourceID ResourceMovement = 0;
static const FGameplayResourceSet::FResourceID ResourceLogic = 1;
static const FGameplayResourceSet::FResourceID ResourceAnimation = 2;

static const FGameplayResourceSet MovementResourceSet = FGameplayResourceSet().AddID(ResourceMovement);
static const FGameplayResourceSet LogicResourceSet = FGameplayResourceSet().AddID(ResourceLogic);
static const FGameplayResourceSet AnimationResourceSet = FGameplayResourceSet().AddID(ResourceAnimation);
static const FGameplayResourceSet MoveAndAnimResourceSet = FGameplayResourceSet().AddSet(MovementResourceSet).AddSet(AnimationResourceSet);
static const FGameplayResourceSet MoveAndLogicResourceSet = FGameplayResourceSet((1 << ResourceMovement) | (1 << ResourceLogic));
static const FGameplayResourceSet MoveAnimLogicResourceSet = FGameplayResourceSet((1 << ResourceMovement) | (1 << ResourceLogic) | (1 << ResourceAnimation));

static const uint8 LowPriority = 1;
static const uint8 HighPriority = 255;

struct FAITest_GameplayTask_ComponentState : public FAITest_GameplayTasksTest
{
	virtual bool InstantTest() override
	{
		AITEST_FALSE("UGameplayTasksComponent\'s default behavior is not to tick initially", Component->GetShouldTick());
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_ComponentState, "System.AI.Gameplay Tasks.Component\'s basic behavior")

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_ExternalCancelWithTick : public FAITest_GameplayTasksTest
{	
	UMockTask_Log* Task;

	virtual bool SetUp() override
	{
		FAITest_GameplayTasksTest::SetUp();

		Logger.ExpectedValues.Add(ETestTaskMessage::Activate);
		Logger.ExpectedValues.Add(ETestTaskMessage::Tick);
		Logger.ExpectedValues.Add(ETestTaskMessage::ExternalCancel);
		Logger.ExpectedValues.Add(ETestTaskMessage::Ended);

		UWorld& World = GetWorld();
		Task = UMockTask_Log::CreateTask(*Component, Logger);
		Task->EnableTick();
		
		AITEST_TRUE("Task should be \'uninitialized\' before Activate is called on it", Task->GetState() == EGameplayTaskState::AwaitingActivation);
		
		Task->ReadyForActivation();
		AITEST_TRUE("Task should be \'Active\' after basic call to ReadyForActivation", Task->GetState() == EGameplayTaskState::Active);
		AITEST_TRUE("Component should want to tick in this scenario", Component->GetShouldTick());

		return true;
	}

	virtual bool InstantTest() override
	{
		TickComponent();
		Task->ExternalCancel();
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_ExternalCancelWithTick, "System.AI.Gameplay Tasks.External Cancel with Tick")

//----------------------------------------------------------------------//
// In this test the task should get properly created, acticated and end 
// during update without any ticking
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_SelfEnd : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Task;

	virtual bool SetUp() override
	{
		FAITest_GameplayTasksTest::SetUp();

		Logger.ExpectedValues.Add(ETestTaskMessage::Activate);
		Logger.ExpectedValues.Add(ETestTaskMessage::Ended);

		UWorld& World = GetWorld();
		Task = UMockTask_Log::CreateTask(*Component, Logger);
		Task->EnableTick();
		Task->ReadyForActivation();
		return true;
	}

	virtual bool InstantTest() override
	{
		Task->EndTask();
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_SelfEnd, "System.AI.Gameplay Tasks.Self End")

//----------------------------------------------------------------------//
// Testing multiple simultaneously ticking tasks
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_SimulatanousTick : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Tasks[3];

	virtual bool SetUp() override
	{
		FAITest_GameplayTasksTest::SetUp();

		Logger.ExpectedValues.Add(ETestTaskMessage::Activate);
		Logger.ExpectedValues.Add(ETestTaskMessage::Activate); 
		Logger.ExpectedValues.Add(ETestTaskMessage::Activate); 
		Logger.ExpectedValues.Add(ETestTaskMessage::Tick);
		Logger.ExpectedValues.Add(ETestTaskMessage::Tick);
		Logger.ExpectedValues.Add(ETestTaskMessage::Tick);
		Logger.ExpectedValues.Add(ETestTaskMessage::ExternalCancel);
		Logger.ExpectedValues.Add(ETestTaskMessage::Ended);
		Logger.ExpectedValues.Add(ETestTaskMessage::ExternalCancel);
		Logger.ExpectedValues.Add(ETestTaskMessage::Ended);
		Logger.ExpectedValues.Add(ETestTaskMessage::ExternalCancel);
		Logger.ExpectedValues.Add(ETestTaskMessage::Ended);

		for (int32 Index = 0; Index < sizeof(Tasks) / sizeof(UMockTask_Log*); ++Index)
		{
			Tasks[Index] = UMockTask_Log::CreateTask(*Component, Logger);
			Tasks[Index]->EnableTick();
			Tasks[Index]->ReadyForActivation();
		}
		AITEST_TRUE("Component should want to tick in this scenario", Component->GetShouldTick());
		return true;
	}

	virtual bool InstantTest() override
	{
		TickComponent();
		for (int32 Index = 0; Index < sizeof(Tasks) / sizeof(UMockTask_Log*); ++Index)
		{
			Tasks[Index]->ExternalCancel();
		}
		AITEST_FALSE("Component should want to tick in this scenario", Component->GetShouldTick());
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_SimulatanousTick, "System.AI.Gameplay Tasks.Simultanously ticking tasks")

//----------------------------------------------------------------------//
// Testing multiple simultaneously ticking tasks UGameplayTaskResource
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_ResourceSet : public FAITestBase
{
	virtual bool InstantTest() override
	{
		FGameplayResourceSet ResourcesSet;
		AITEST_TRUE("New FGameplayResourceSet should be empty initialy", ResourcesSet.IsEmpty());
		ResourcesSet.AddID(ResourceLogic);
		AITEST_FALSE("Added one ID, ResourcesSet should not be perceived as empty now", ResourcesSet.IsEmpty());
		ResourcesSet.RemoveID(ResourceAnimation);
		AITEST_FALSE("Removed ID not previously added, ResourcesSet should not be perceived as empty now", ResourcesSet.IsEmpty());
		ResourcesSet.RemoveID(ResourceLogic);
		AITEST_TRUE("Removed ID previously added, ResourcesSet should be empty now", ResourcesSet.IsEmpty());

		AITEST_FALSE("Single ID checking, not present ID", MoveAndAnimResourceSet.HasAnyID(LogicResourceSet));
		AITEST_TRUE("Single ID checking", MoveAndAnimResourceSet.HasAnyID(MovementResourceSet));
		AITEST_TRUE("Single ID checking", MoveAndAnimResourceSet.HasAnyID(AnimationResourceSet));

		AITEST_TRUE("Multiple ID checking - has all, self test", MoveAndAnimResourceSet.HasAllIDs(MoveAndAnimResourceSet));
		AITEST_TRUE("Multiple ID checking - has all, other identical", MoveAndAnimResourceSet.HasAllIDs(FGameplayResourceSet((1 << ResourceMovement) | (1 << ResourceAnimation))));
		AITEST_FALSE("Multiple ID checking - has all, other different", MoveAndAnimResourceSet.HasAllIDs(MoveAndLogicResourceSet));
		AITEST_TRUE("Multiple ID checking - overlap", MoveAndAnimResourceSet.GetOverlap(MoveAndLogicResourceSet) == MovementResourceSet);
		AITEST_TRUE("Multiple ID checking - substraction", MoveAndAnimResourceSet.GetDifference(MoveAndLogicResourceSet) == AnimationResourceSet);
		
		AITEST_FALSE("FGameplayResourceSet containing 0-th ID is not empty", MovementResourceSet.IsEmpty());
		AITEST_TRUE("FGameplayResourceSet has 0-th ID", MovementResourceSet.HasID(ResourceMovement));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_ResourceSet, "System.AI.Gameplay Tasks.Resource Set")

//----------------------------------------------------------------------//
// Running tasks requiring non-overlapping resources
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_NonOverlappingResources : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Tasks[2];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();
		
		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndAnimResourceSet);
		Tasks[1] = UMockTask_Log::CreateTask(*Component, Logger, LogicResourceSet);

		Tasks[0]->ReadyForActivation();
		AITEST_TRUE("TasksComponent should claim it's using 0th task's resources", Component->GetCurrentlyUsedResources() == Tasks[0]->GetClaimedResources());
		
		Tasks[1]->ReadyForActivation();		
		AITEST_TRUE("Both tasks should be \'Active\' since their resources do not overlap", Tasks[0]->GetState() == EGameplayTaskState::Active && Tasks[1]->GetState() == EGameplayTaskState::Active);
		AITEST_TRUE("TasksComponent should claim it's using only latter task's resources", Component->GetCurrentlyUsedResources() == MoveAnimLogicResourceSet);

		Tasks[0]->ExternalCancel();
		AITEST_TRUE("Only index 1 task's resources should be relevant now", Component->GetCurrentlyUsedResources() == Tasks[1]->GetClaimedResources());

		Tasks[1]->ExternalCancel();
		AITEST_TRUE("No resources should be occupied now", Component->GetCurrentlyUsedResources().IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_NonOverlappingResources, "System.AI.Gameplay Tasks.Non-overlapping resources")

//----------------------------------------------------------------------//
// Running tasks requiring overlapping resources
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_OverlappingResources : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Tasks[2];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndAnimResourceSet);
		Tasks[1] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndLogicResourceSet);

		Tasks[0]->ReadyForActivation();
		Tasks[1]->ReadyForActivation();

		AITEST_TRUE("Only the latter task should be active since it shadows the other one in terms of required resources", Tasks[1]->GetState() == EGameplayTaskState::Active);
		AITEST_TRUE("The first task should be paused at this moment", Tasks[0]->GetState() == EGameplayTaskState::Paused);
		AITEST_TRUE("TasksComponent should claim it's using only latter task's resources", Component->GetCurrentlyUsedResources() == Tasks[1]->GetClaimedResources());

		Tasks[1]->ExternalCancel();
		AITEST_TRUE("Now the latter task should be marked as Finished", Tasks[1]->GetState() == EGameplayTaskState::Finished);
		AITEST_TRUE("And the first task should be resumed", Tasks[0]->GetState() == EGameplayTaskState::Active);
		AITEST_TRUE("TasksComponent should claim it's using only first task's resources", Component->GetCurrentlyUsedResources() == Tasks[0]->GetClaimedResources());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_OverlappingResources, "System.AI.Gameplay Tasks.Overlapping resources")

//----------------------------------------------------------------------//
// Pausing a task overlapping a lower priority task should not resume the low priority task
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_PausingTasksBlockingOtherTasks : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Tasks[3];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);
		Tasks[1] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndLogicResourceSet);
		Tasks[2] = UMockTask_Log::CreateTask(*Component, Logger, LogicResourceSet);
		 
		Tasks[0]->ReadyForActivation();
		Tasks[1]->ReadyForActivation();

		AITEST_FALSE("First task should be paused since it's resources get overlapped", Tasks[0]->IsActive());
		AITEST_TRUE("Second task should on top and active", Tasks[1]->IsActive());

		Tasks[2]->ReadyForActivation();
		AITEST_FALSE("Second task should get paused since its resources got overlapped", Tasks[1]->IsActive());
		AITEST_FALSE("First task should remain paused since it's resources get overlapped by the paused task", Tasks[0]->IsActive());

		Tasks[2]->ExternalCancel();
		AITEST_FALSE("Nothing shoud change for the first task", Tasks[0]->IsActive());
		AITEST_TRUE("Second task should be active again", Tasks[1]->IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_PausingTasksBlockingOtherTasks, "System.AI.Gameplay Tasks.Pausing tasks blocking other tasks")

//----------------------------------------------------------------------//
// Priority handling
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_Priorities : public FAITest_GameplayTasksTest
{
	UMockTask_Log* Tasks[3];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		// all tasks use same resources, have different priorities
		// let's do some tick testing as well
		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndAnimResourceSet, LowPriority);
		Tasks[0]->EnableTick();
		Tasks[1] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndAnimResourceSet);
		Tasks[1]->EnableTick();
		Tasks[2] = UMockTask_Log::CreateTask(*Component, Logger, MoveAndAnimResourceSet, HighPriority);		

		Tasks[1]->ReadyForActivation();
		Tasks[0]->ReadyForActivation();

		AITEST_TRUE("Task at index 1 should be active at this point since it's higher priority", Tasks[1]->IsActive() && !Tasks[0]->IsActive());
		AITEST_TRUE("TasksComponent should claim it's using only resources of task 1", Component->GetCurrentlyUsedResources() == Tasks[1]->GetClaimedResources());
		AITEST_TRUE("Current top action wants to tick so Component should want that as well", Component->GetShouldTick());

		Tasks[2]->ReadyForActivation();
		AITEST_TRUE("Now the last pushed, highest priority task should be active", Tasks[2]->IsActive() && !Tasks[0]->IsActive() && !Tasks[1]->IsActive());
		AITEST_FALSE("No ticking task is active so Component should not want to tick", Component->GetShouldTick());
		
		Tasks[1]->ExternalCancel();
		AITEST_TRUE("Canceling mid-priority inactive task should not influence what's active", Tasks[2]->IsActive() && !Tasks[0]->IsActive() && !Tasks[1]->IsActive());
		AITEST_FALSE("Current top action still doesn't want to tick, so neither should the Component", Component->GetShouldTick());

		Tasks[2]->ExternalCancel();
		AITEST_TRUE("After canceling the top-priority task the lowest priority task remains to be active", !Tasks[2]->IsActive() && Tasks[0]->IsActive() && !Tasks[1]->IsActive());
		AITEST_TRUE("New top action wants tick, so should Component", Component->GetShouldTick());

		Tasks[0]->ExternalCancel();
		AITEST_FALSE("Task-less component should not want to tick", Component->GetShouldTick());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_Priorities, "System.AI.Gameplay Tasks.Priorities")

//----------------------------------------------------------------------//
// Internal ending, by task ending itself or owner finishing 
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_InternalEnding : public FAITest_GameplayTasksTest
{
	static const int32 TasksCount = 4;
	UMockTask_Log* Task_SelfEndNoResources;
	UMockTask_Log* Task_OwnerEndNoResources;
	UMockTask_Log* Task_SelfEndWithResources;
	UMockTask_Log* Task_OwnerEndWithResources;
	UMockTask_Log* Tasks[TasksCount];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		// all tasks use same resources, have different priorities
		Tasks[0] = Task_SelfEndNoResources = UMockTask_Log::CreateTask(*Component, Logger);
		Tasks[1] = Task_OwnerEndNoResources = UMockTask_Log::CreateTask(*Component, Logger);
		// not using overlapping resources set on purpose - want to test them independently
		Tasks[2] = Task_SelfEndWithResources = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);
		Tasks[3] = Task_OwnerEndWithResources = UMockTask_Log::CreateTask(*Component, Logger, LogicResourceSet);
		
		for (int32 Index = 0; Index < TasksCount; ++Index)
		{
			Tasks[Index]->ReadyForActivation();
			AITEST_TRUE("Trivial activation should succeed", Tasks[Index]->IsActive());
		}
		AITEST_TRUE("Resources should sum up", Component->GetCurrentlyUsedResources() == MoveAndLogicResourceSet);

		Task_SelfEndNoResources->EndTask();
		AITEST_FALSE("Task_SelfEndNoResources should be \'done\' now", Task_SelfEndNoResources->IsActive());

		Task_OwnerEndNoResources->TaskOwnerEnded();
		AITEST_FALSE("Task_SelfEndNoResources should be \'done\' now", Task_OwnerEndNoResources->IsActive());

		Task_SelfEndWithResources->EndTask();
		AITEST_FALSE("Task_SelfEndWithResources should be \'done\' now", Task_SelfEndWithResources->IsActive());
		AITEST_TRUE("Only the other task's resources should matter now", Component->GetCurrentlyUsedResources() == LogicResourceSet);
		AITEST_TRUE("There should be only one active task in the priority queue", Component->GetTaskPriorityQueueSize() == 1);

		Task_OwnerEndWithResources->TaskOwnerEnded();
		AITEST_FALSE("Task_SelfEndWithResources should be \'done\' now", Task_OwnerEndWithResources->IsActive());
		AITEST_TRUE("No resources should be locked at this moment", Component->GetCurrentlyUsedResources().IsEmpty());
		AITEST_TRUE("Priority Task Queue should be empty", Component->GetTaskPriorityQueueSize() == 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_InternalEnding, "System.AI.Gameplay Tasks.Self and Owner ending")

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_MultipleOwners : public FAITest_GameplayTasksTest
{
	static const int32 TasksCount = 3;
	UMockTask_Log* Tasks[TasksCount];
	UMockTask_Log* LowPriorityTask;
	UMockGameplayTaskOwner* OtherOwner;

	virtual bool InstantTest() override
	{
		OtherOwner = NewObject<UMockGameplayTaskOwner>();
		OtherOwner->GTComponent = Component;
		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);
		Tasks[1] = UMockTask_Log::CreateTask(*OtherOwner, Logger, MovementResourceSet);
		Tasks[2] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);

		for (int32 Index = 0; Index < TasksCount; ++Index)
		{
			Tasks[Index]->ReadyForActivation();
		}
		// This part tests what happens if "other owner" task is in the middle of the queue and
		// not active
		AITEST_TRUE("Last pushed task should be active now", Tasks[2]->IsActive());
		Component->EndAllResourceConsumingTasksOwnedBy(*Component);
		AITEST_TRUE("There should be only one task in the queue now", Component->GetTaskPriorityQueueSize() == 1);
		AITEST_TRUE("The last remaining task should be active now", Tasks[1]->IsActive());

		// this part tests what happens during pruning if the "other owner" task is active at the
		// moment of performing the action
		LowPriorityTask = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet, LowPriority);
		LowPriorityTask->ReadyForActivation();
		AITEST_TRUE("There should be 2 tasks in the queue now", Component->GetTaskPriorityQueueSize() == 2);
		Component->EndAllResourceConsumingTasksOwnedBy(*Component);
		AITEST_TRUE("There should be only one task in the queue after second pruning", Component->GetTaskPriorityQueueSize() == 1);
		AITEST_TRUE("The last remaining task should be still active", Tasks[1]->IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_MultipleOwners, "System.AI.Gameplay Tasks.Handling multiple task owners")

//----------------------------------------------------------------------//
// Claimed vs Required resources test
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_ClaimedResources : public FAITest_GameplayTasksTest
{
	static const int32 TasksCount = 4;
	UMockTask_Log* Tasks[TasksCount];

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		// create three tasks
		// first one has a resource we're going to overlap with the extra-claimed resource of the next task
		Tasks[0] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);
		Tasks[0]->ReadyForActivation();

		// second task requires non-overlapping resources to first task
		Tasks[1] = UMockTask_Log::CreateTask(*Component, Logger, AnimationResourceSet);
		// but declared an overlapping resource as "claimed"
		Tasks[1]->AddClaimedResourceSet(MovementResourceSet);
		
		Tasks[1]->ReadyForActivation();
		// at this point first task should get paused since it's required resource is claimed, or shadowed, by the newer task
		AITEST_FALSE("The first task should get paused since its required resource is claimed, or shadowed, by the newer task", Tasks[0]->IsActive());
		AITEST_TRUE("The second task should be running, nothing obstructing it", Tasks[1]->IsActive());
		
		// a new low-priority task should not be allowed to run neither
		Tasks[2] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet, LowPriority);
		Tasks[2]->ReadyForActivation();
		AITEST_FALSE("The new low-priority task should not be allowed to run neither", Tasks[2]->IsActive());
		AITEST_TRUE("The second task should be still running", Tasks[1]->IsActive());

		// however, a new task, that's using the overlapped claimed resource 
		// should run without any issues
		// note, this doesn't have to be "high priority" task - new tasks with same priority as "current"
		// are treated like higher priority anyway
		Tasks[3] = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet, HighPriority);
		Tasks[3]->ReadyForActivation();
		AITEST_TRUE("The new high-priority task should be allowed to run", Tasks[3]->IsActive());
		// but active task, that declared the active resources should not get paused neither
		AITEST_TRUE("The second task should be still running, it's required resources are not being overlapped", Tasks[1]->IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_ClaimedResources, "System.AI.Gameplay Tasks.Claimed resources")

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
struct FAITest_GameplayTask_ClaimedResourcesAndInstantFinish : public FAITest_GameplayTasksTest
{
	static const int32 TasksCount = 4;
	UMockTask_Log* Task;

	virtual bool InstantTest() override
	{
		UWorld& World = GetWorld();

		Task = UMockTask_Log::CreateTask(*Component, Logger, MovementResourceSet);
		Task->SetInstaEnd(true);
		Task->ReadyForActivation();

		AITEST_TRUE("No claimed resources should be left behind", Component->GetCurrentlyUsedResources().IsEmpty());
		AITEST_TRUE("There should no active tasks when task auto-insta-ended", Component->GetTaskPriorityQueueSize() == 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_GameplayTask_ClaimedResourcesAndInstantFinish, "System.AI.Gameplay Tasks.Claimed resources vs Insta-finish tasks")

// add tests if component wants ticking at while aborting/reactivating tasks
// add test for re-adding/re-activating a finished task

#undef LOCTEXT_NAMESPACE
