// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AITypes.h"
#include "AITestsCommon.h"

//----------------------------------------------------------------------//
// FAITest_ResourceIDBasic
//----------------------------------------------------------------------//
struct FAITest_ResourceIDBasic : public FAITestBase
{
	virtual bool InstantTest() override
	{
		AITEST_TRUE("There are always some resources as long as AIModule is present", FAIResources::GetResourcesCount() > 0);

		const FAIResourceID& MovementID = FAIResources::GetResource(FAIResources::Movement);
		AITEST_TRUE("Resource ID's indexes are broken!", FAIResources::Movement == MovementID);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_ResourceIDBasic, "System.AI.Resource ID.Basic operations")

//----------------------------------------------------------------------//
// FAITest_ResourceLock
//----------------------------------------------------------------------//
struct FAITest_ResourceLock : public FAITestBase
{
	virtual bool InstantTest() override
	{
		FAIResourceLock MockLock;

		// basic locking
		MockLock.SetLock(EAIRequestPriority::HardScript);
		AITEST_TRUE("Resource should be locked", MockLock.IsLocked());
		AITEST_TRUE("Resource should be locked with specified priority", MockLock.IsLockedBy(EAIRequestPriority::HardScript));
		AITEST_FALSE("Resource should not be available for lower priorities", MockLock.IsAvailableFor(EAIRequestPriority::Logic));
		AITEST_TRUE("Resource should be available for higher priorities", MockLock.IsAvailableFor(EAIRequestPriority::Reaction));

		// clearing lock:
		// try clearing with lower priority
		MockLock.ClearLock(EAIRequestPriority::Logic);
		AITEST_TRUE("Resource should be still locked", MockLock.IsLocked());
		AITEST_FALSE("Resource should still not be available for lower priorities", MockLock.IsAvailableFor(EAIRequestPriority::Logic));
		AITEST_TRUE("Resource should still be available for higher priorities", MockLock.IsAvailableFor(EAIRequestPriority::Reaction));

		// releasing the actual lock
		MockLock.ClearLock(EAIRequestPriority::HardScript);
		AITEST_FALSE("Resource should be available now", MockLock.IsLocked());

		// clearing all locks at one go
		MockLock.SetLock(EAIRequestPriority::HardScript);
		MockLock.SetLock(EAIRequestPriority::Logic);
		MockLock.SetLock(EAIRequestPriority::Reaction);
		bool bWasLocked = MockLock.IsLocked();
		MockLock.ForceClearAllLocks();
		AITEST_TRUE("Resource should no longer be locked", bWasLocked == true && MockLock.IsLocked() == false);

		// merging
		FAIResourceLock MockLock2;
		MockLock.SetLock(EAIRequestPriority::HardScript);
		MockLock2.SetLock(EAIRequestPriority::Logic);
		// merge
		MockLock2 += MockLock;
		AITEST_TRUE("Resource should be locked on both priorities", MockLock2.IsLockedBy(EAIRequestPriority::Logic) && MockLock2.IsLockedBy(EAIRequestPriority::HardScript));
		MockLock2.ClearLock(EAIRequestPriority::Logic);
		AITEST_TRUE("At this point both locks should be identical", MockLock == MockLock2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_ResourceLock, "System.AI.Resource ID.Resource locking")

//----------------------------------------------------------------------//
// FAITest_ResourceSet
//----------------------------------------------------------------------//
struct FAITest_ResourceSet : public FAITestBase
{
	virtual bool InstantTest() override
	{
		{
			FAIResourcesSet ResourceSet;
			AITEST_TRUE("Resource Set should be empty by default", ResourceSet.IsEmpty());
			for (uint8 FlagIndex = 0; FlagIndex < FAIResourcesSet::MaxFlags; ++FlagIndex)
			{
				AITEST_FALSE("Resource Set should not contain any resources when empty", ResourceSet.ContainsResourceIndex(FlagIndex));
			}
		}

		{
			FAIResourcesSet ResourceSet(FAIResourcesSet::AllResources);
			AITEST_FALSE("Resource Set should be empty by default", ResourceSet.IsEmpty());
			for (uint8 FlagIndex = 0; FlagIndex < FAIResourcesSet::MaxFlags; ++FlagIndex)
			{
				AITEST_TRUE("Full Resource Set should contain every resource", ResourceSet.ContainsResourceIndex(FlagIndex) == true);
			}
		}

		{
			const FAIResourceID& MovementResource = FAIResources::GetResource(FAIResources::Movement);
			const FAIResourceID& PerceptionResource = FAIResources::GetResource(FAIResources::Perception);

			FAIResourcesSet ResourceSet;
			ResourceSet.AddResource(PerceptionResource);
			AITEST_TRUE("Resource Set should contain added resource", ResourceSet.ContainsResource(PerceptionResource));
			AITEST_TRUE("Resource Set should contain added resource given by Index", ResourceSet.ContainsResourceIndex(PerceptionResource.Index));
			for (uint8 FlagIndex = 0; FlagIndex < FAIResourcesSet::MaxFlags; ++FlagIndex)
			{
				if (FlagIndex != PerceptionResource.Index)
				{
					AITEST_FALSE("Resource Set should not contain any other resources", ResourceSet.ContainsResourceIndex(FlagIndex));
				}
			}
			AITEST_FALSE("Resource Set should not be empty after adding a resource", ResourceSet.IsEmpty());

			ResourceSet.AddResourceIndex(MovementResource.Index);
			AITEST_TRUE("Resource Set should contain second added resource", ResourceSet.ContainsResource(MovementResource));
			AITEST_TRUE("Resource Set should contain second added resource given by Index", ResourceSet.ContainsResourceIndex(MovementResource.Index));

			ResourceSet.RemoveResource(MovementResource);
			AITEST_FALSE("Resource Set should no longer contain second added resource", ResourceSet.ContainsResource(MovementResource));
			AITEST_FALSE("Resource Set should still be not empty after removing one resource", ResourceSet.IsEmpty());

			ResourceSet.RemoveResourceIndex(PerceptionResource.Index);
			AITEST_TRUE("Resource Set should be empty after removing last resource", ResourceSet.IsEmpty() == true);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FAITest_ResourceSet, "System.AI.Resource ID.Resource locking")
