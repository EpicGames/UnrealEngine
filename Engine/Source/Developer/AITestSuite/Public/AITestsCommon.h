// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Package.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/UObjectGlobals.h"
#include "CoreGlobals.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Misc/AutomationTest.h"
#include "TestLogger.h"
#include "Engine/EngineBaseTypes.h"
#include "TestableEnsures.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAITestSuite, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogBehaviorTreeTest, Log, All);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FAITestCommand_WaitSeconds, float, Duration);

namespace UE::AITestSuite
{
	/** Will cause UE_DEBUG_BREAK if `test.BreakOnTestFail` console variable is true */
	AITESTSUITE_API void ConditionallyBreakOnTestFail(); 
}

class FAITestCommand_WaitOneTick : public IAutomationLatentCommand
{
public: 
	FAITestCommand_WaitOneTick()
		: bAlreadyRun(false)
	{} 
	virtual bool Update() override;
private: 
	bool bAlreadyRun;
};

namespace FAITestHelpers
{
	AITESTSUITE_API UWorld* GetWorld();
	static constexpr float TickInterval = 1.f / 30;

	void UpdateFrameCounter();
	uint64 FramesCounter();
}

struct FAITestBase
{
private:
	// internals
	TArray<UObject*> SpawnedObjects;
	uint32 bTearedDown : 1;
protected:
	FAutomationTestBase* TestRunner;

	FAITestBase() : bTearedDown(false), TestRunner(nullptr)
	{}

	template<typename ClassToSpawn>
	ClassToSpawn* NewAutoDestroyObject(UObject* Outer = GetTransientPackage())
	{
		ClassToSpawn* ObjectInstance = NewObject<ClassToSpawn>(Outer);
		ObjectInstance->AddToRoot();
		SpawnedObjects.Add(ObjectInstance);
		return ObjectInstance;
	}

	AITESTSUITE_API void AddAutoDestroyObject(UObject& ObjectRef);
	AITESTSUITE_API virtual UWorld& GetWorld() const;

	FAutomationTestBase& GetTestRunner() const { check(TestRunner); return *TestRunner; }

public:

	virtual void SetTestRunner(FAutomationTestBase& AutomationTestInstance) { TestRunner = &AutomationTestInstance; }

	// interface
	AITESTSUITE_API virtual ~FAITestBase();
	/** @return true if setup was completed successfully, false otherwise (which will result in failing the test instance). */
	virtual bool SetUp() { return true; }
	/** @return true to indicate that the test is done. */
	virtual bool Update() { return false; } 
	/** lets the Test instance test the results. Use AITEST_*_LATENT macros */
	virtual void VerifyLatentResults() {}
	/** @return false to indicate an issue with test execution. Will signal to automation framework this test instance failed. */
	virtual bool InstantTest() { return false;}
	// it's essential that overriding functions call the super-implementation. Otherwise, the check in ~FAITestBase will fail.
	AITESTSUITE_API virtual void TearDown();
};

DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(AITESTSUITE_API, FAITestCommand_SetUpTest, FAITestBase*, AITest);
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(AITESTSUITE_API, FAITestCommand_PerformTest, FAITestBase*, AITest);
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(AITESTSUITE_API, FAITestCommand_VerifyTestResults, FAITestBase*, AITest);
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(AITESTSUITE_API, FAITestCommand_TearDownTest, FAITestBase*, AITest);

// @note that TestClass needs to derive from FAITestBase
#define IMPLEMENT_AI_LATENT_TEST(TestClass, PrettyName) \
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TestClass##_Runner, PrettyName, (EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)) \
	bool TestClass##_Runner::RunTest(const FString& Parameters) \
	{ \
		/* spawn test instance. Setup should be done in test's constructor */ \
		TestClass* TestInstance = new TestClass(); \
		TestInstance->SetTestRunner(*this); \
		/* set up */ \
		ADD_LATENT_AUTOMATION_COMMAND(FAITestCommand_SetUpTest(TestInstance)); \
		/* run latent command to update */ \
		ADD_LATENT_AUTOMATION_COMMAND(FAITestCommand_PerformTest(TestInstance)); \
		/* let the Test instance verify the results, calls VerifyLatentResults */ \
		ADD_LATENT_AUTOMATION_COMMAND(FAITestCommand_VerifyTestResults(TestInstance)); \
		/* run latent command to tear down */ \
		ADD_LATENT_AUTOMATION_COMMAND(FAITestCommand_TearDownTest(TestInstance)); \
		return true; \
	} 

#define IMPLEMENT_AI_INSTANT_TEST_WITH_FLAGS(TestClass, PrettyName, TestFlags) \
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TestClass##Runner, PrettyName, (TestFlags)) \
	bool TestClass##Runner::RunTest(const FString& Parameters) \
	{ \
		bool bSuccess = false; \
		/* spawn test instance. */ \
		TestClass* TestInstance = new TestClass(); \
		TestInstance->SetTestRunner(*this); \
		/* set up */ \
		if (TestInstance->SetUp()) \
		{ \
			/* call the instant-test code */ \
			bSuccess = TestInstance->InstantTest(); \
		}\
		/* tear down */ \
		TestInstance->TearDown(); \
		delete TestInstance; \
		return bSuccess; \
	} 

#define IMPLEMENT_AI_INSTANT_TEST(TestClass, PrettyName) \
	IMPLEMENT_AI_INSTANT_TEST_WITH_FLAGS(TestClass, PrettyName, EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/** 
 *	This macro allows one to implement a whole set of simple tests that share common setups. To use it first implement
 *	a struct that builds the said common setup. Like so:
 *
 *		struct FMyCommonSetup : public FAITestBase
 *		{
 *			virtual bool SetUp() override
 *			{
 *				// your test common setup build code here
 *
 *				// return false if setup fails and the test needs to be aborted
 *				return true; 
 *			}
 *		};
 *	
 *	Once that's done you can implement a specific test using this setup class like so:
 *
 *	IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMyCommonSetup, "System.Engine.AI.MyTestGroup", ThisSpecificTestName)
 *	{
 *		// your test code here
 *
 *		// return false to indicate the whole test instance failed for some reason
 *		return true;
 *	}
 */
#define IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(Fixture, PrettyGroupNameString, TestExperiment) \
	struct Fixture##_##TestExperiment : public Fixture \
	{ \
		virtual bool InstantTest() override; \
	}; \
	IMPLEMENT_AI_INSTANT_TEST(Fixture##_##TestExperiment, PrettyGroupNameString "." # TestExperiment) \
	bool Fixture##_##TestExperiment::InstantTest()
//----------------------------------------------------------------------//
// Specific test types
//----------------------------------------------------------------------//
template<class FReal>
struct FAITest_SimpleComponentBasedTest : public FAITestBase
{
	FTestLogger<int32> Logger;
	FReal* Component;

	FAITest_SimpleComponentBasedTest()
	{
		Component = NewAutoDestroyObject<FReal>();
	}

	virtual void SetTestRunner(FAutomationTestBase& AutomationTestInstance) override
	{ 
		FAITestBase::SetTestRunner(AutomationTestInstance);
		Logger.TestRunner = TestRunner;
	}

	virtual ~FAITest_SimpleComponentBasedTest() override
	{
		GetTestRunner().TestTrue(TEXT("Not all expected values has been logged"), Logger.ExpectedValues.Num() == 0 || Logger.ExpectedValues.Num() == Logger.LoggedValues.Num());
	}

	virtual bool SetUp() override
	{
		UWorld* World = FAITestHelpers::GetWorld();
		Component->RegisterComponentWithWorld(World);
		return World != nullptr;
	}

	void TickComponent()
	{
		Component->TickComponent(FAITestHelpers::TickInterval, ELevelTick::LEVELTICK_All, nullptr);
	}
};

//----------------------------------------------------------------------//
// state testing macros, valid in FTestAIBase (and subclasses') methods 
// Using these macros makes sure the test function fails if the assertion
// fails making sure the rest of the test relying on given condition being 
// true doesn't crash
//----------------------------------------------------------------------//
#define __AITEST_IMPL(_TestRunner, What, Value, Test, RetVal)\
	if (!_TestRunner.Test(What, Value))\
	{\
		UE::AITestSuite::ConditionallyBreakOnTestFail(); \
		return RetVal;\
	}
	
#define AITEST_TRUE_EX(_TestRunner, What, Value) __AITEST_IMPL(_TestRunner, What, Value, TestTrue, false)
#define AITEST_FALSE_EX(_TestRunner, What, Value) __AITEST_IMPL(_TestRunner, What, Value, TestFalse, false)
#define AITEST_NULL_EX(_TestRunner, What, Pointer) __AITEST_IMPL(_TestRunner, What, Pointer, TestNull, false)
#define AITEST_NOT_NULL_EX(_TestRunner, What, Pointer) \
	__AITEST_IMPL(_TestRunner, What, Pointer, TestNotNull, false); \
	CA_ASSUME(Pointer)

#define AITEST_TRUE(What, Value) AITEST_TRUE_EX(GetTestRunner(), What, Value)
#define AITEST_FALSE(What, Value) AITEST_FALSE_EX(GetTestRunner(), What, Value)
#define AITEST_NULL(What, Pointer) AITEST_NULL_EX(GetTestRunner(), What, Pointer)
#define AITEST_NOT_NULL(What, Pointer) AITEST_NOT_NULL_EX(GetTestRunner(), What, Pointer)

namespace FTestHelpers
{
	template<typename T1, typename T2>
	inline bool TestEqual(const FString& Description, T1 Expression, T2 Expected, FAutomationTestBase& This)
	{
		return This.TestEqual(*Description, Expression, Expected);
	}

	template<typename T1, typename T2>
	inline bool TestEqual(const FString& Description, T1* Expression, T2* Expected, FAutomationTestBase& This)
	{
		return This.TestEqual(*Description, reinterpret_cast<uint64>(Expression), reinterpret_cast<uint64>(Expected));
	}

	template<typename T1, typename T2>
	inline bool TestNotEqual(const FString& Description, T1 Expression, T2 Expected, FAutomationTestBase& This)
	{
		return This.TestNotEqual(*Description, Expression, Expected);
	}

	template<typename T1, typename T2>
	inline bool TestNotEqual(const FString& Description, T1* Expression, T2* Expected, FAutomationTestBase& This)
	{
		return This.TestNotEqual(*Description, reinterpret_cast<uint64>(Expression), reinterpret_cast<uint64>(Expected));
	}

	inline void AddInfo(const FString& Info, FAutomationTestBase& This)
	{
		This.AddInfo(*Info);
	}
}

#define __AITEST_HELPER_IMPL(_TestRunner, What, Actual, Expected, Test, RetVal)\
	if (!FTestHelpers::Test(What, Actual, Expected, _TestRunner))\
	{\
		UE::AITestSuite::ConditionallyBreakOnTestFail(); \
		return RetVal;\
	}

#define AITEST_EQUAL_EX(_TestRunner, What, Actual, Expected) __AITEST_HELPER_IMPL(_TestRunner, What, Actual, Expected, TestEqual, false)
#define AITEST_NOT_EQUAL_EX(_TestRunner, What, Actual, Expected) __AITEST_HELPER_IMPL(_TestRunner, What, Actual, Expected, TestNotEqual, false)
#define AITEST_INFO_EX(_TestRunner, InfoString) FTestHelpers::AddInfo(InfoString, _TestRunner)

#define AITEST_EQUAL(What, Actual, Expected) AITEST_EQUAL_EX(GetTestRunner(), What, Actual, Expected)
#define AITEST_NOT_EQUAL(What, Actual, Expected) AITEST_NOT_EQUAL_EX(GetTestRunner(), What, Actual, Expected)
#define AITEST_INFO(InfoString) AITEST_INFO_EX(GetTestRunner(), InfoString)


//----------------------------------------------------------------------//
// state testing macros, valid in FTestAIBase (and subclasses') methods. 
// Using these macros makes sure the test function fails if the assertion
// fails making sure the rest of the test relying on given condition being 
// true doesn't crash. Note that these macros are intended to be put in 
// a void-returning function where latent test's results can be verified. 
// Update function is not a good place for testing results since its return
// value controls whether the test will continue.
//----------------------------------------------------------------------//
#define AITEST_TRUE_LATENT(What, Value) __AITEST_IMPL(GetTestRunner(), What, Value, TestTrue, )
#define AITEST_FALSE_LATENT(What, Value) __AITEST_IMPL(GetTestRunner(), What, Value, TestFalse, )
#define AITEST_NULL_LATENT(What, Pointer) __AITEST_IMPL(GetTestRunner(), What, Pointer, TestNull, )
#define AITEST_NOT_NULL_LATENT(What, Pointer) __AITEST_IMPL(GetTestRunner(), What, Pointer, TestNotNull, )
#define AITEST_EQUAL_LATENT(What, Actual, Expected) __AITEST_HELPER_IMPL(GetTestRunner(), What, Actual, Expected, TestEqual, )
#define AITEST_NOT_EQUAL_LATENT(What, Actual, Expected) __AITEST_HELPER_IMPL(GetTestRunner(), What, Actual, Expected, TestNotEqual, )
