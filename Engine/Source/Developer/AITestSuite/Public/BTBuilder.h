// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Composites/BTComposite_Selector.h"
#include "BehaviorTree/Composites/BTComposite_Sequence.h"
#include "BehaviorTree/Composites/BTComposite_SimpleParallel.h"
#include "BehaviorTree/Decorators/BTDecorator_Loop.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"

#include "BehaviorTree/TestBTDecorator_Blackboard.h"
#include "BehaviorTree/TestBTDecorator_Blueprint.h"
#include "BehaviorTree/TestBTDecorator_DelayedAbort.h"
#include "BehaviorTree/TestBTDecorator_LoopUntil.h"
#include "BehaviorTree/TestBTDecorator_NodeMemoryValidator.h"
#include "BehaviorTree/TestBTService_Log.h"
#include "BehaviorTree/TestBTService_BTStopAction.h"
#include "BehaviorTree/TestBTTask_LatentWithFlags.h"
#include "BehaviorTree/TestBTTask_Log.h"
#include "BehaviorTree/TestBTTask_SetFlag.h"
#include "BehaviorTree/TestBTTask_SetValue.h"
#include "BehaviorTree/TestBTTask_SetValuesWithLogs.h"
#include "BehaviorTree/TestBTTask_BTStopAction.h"
#include "BehaviorTree/TestBTTask_ToggleFlag.h"

struct FBTBuilder
{
	static UBehaviorTree& CreateBehaviorTree()
	{
		UBlackboardData* BB = NewObject<UBlackboardData>();
		FBlackboardEntry KeyData;

		KeyData.EntryName = TEXT("Bool1");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Bool>();
		BB->Keys.Add(KeyData);

		KeyData.EntryName = TEXT("Bool2");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Bool>();
		BB->Keys.Add(KeyData);

		KeyData.EntryName = TEXT("Bool3");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Bool>();
		BB->Keys.Add(KeyData);

		KeyData.EntryName = TEXT("Bool4");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Bool>();
		BB->Keys.Add(KeyData);

		KeyData.EntryName = TEXT("Int");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Int>();
		BB->Keys.Add(KeyData);

		KeyData.EntryName = TEXT("Int2");
		KeyData.KeyType = NewObject<UBlackboardKeyType_Int>();
		BB->Keys.Add(KeyData);

		BB->UpdateParentKeys();

		UBehaviorTree* TreeOb = NewObject<UBehaviorTree>();
		TreeOb->BlackboardAsset = BB;

		return *TreeOb;
	}

	static UBehaviorTree& CreateBehaviorTree(UBehaviorTree& ParentTree)
	{
		UBehaviorTree* TreeOb = NewObject<UBehaviorTree>();
		TreeOb->BlackboardAsset = ParentTree.BlackboardAsset;
		
		return *TreeOb;
	}

	static UBTComposite_Selector& AddSelector(UBehaviorTree& TreeOb)
	{
		UBTComposite_Selector* NodeOb = NewObject<UBTComposite_Selector>(&TreeOb);
		NodeOb->InitializeFromAsset(TreeOb);
		TreeOb.RootNode = NodeOb;
		return *NodeOb;
	}

	static UBTComposite_Selector& AddSelector(UBTCompositeNode& ParentNode)
	{
		UBTComposite_Selector* NodeOb = NewObject<UBTComposite_Selector>(ParentNode.GetTreeAsset());
		NodeOb->InitializeFromAsset(*ParentNode.GetTreeAsset());

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildComposite = NodeOb;

		return *NodeOb;
	}

	static UBTComposite_Sequence& AddSequence(UBehaviorTree& TreeOb)
	{
		UBTComposite_Sequence* NodeOb = NewObject<UBTComposite_Sequence>(&TreeOb);
		NodeOb->InitializeFromAsset(TreeOb);
		TreeOb.RootNode = NodeOb;
		return *NodeOb;
	}

	static UBTComposite_Sequence& AddSequence(UBTCompositeNode& ParentNode)
	{
		UBTComposite_Sequence* NodeOb = NewObject<UBTComposite_Sequence>(ParentNode.GetTreeAsset());
		NodeOb->InitializeFromAsset(*ParentNode.GetTreeAsset());

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildComposite = NodeOb;

		return *NodeOb;
	}

	static UBTComposite_SimpleParallel& AddParallel(UBehaviorTree& TreeOb, EBTParallelMode::Type Mode)
	{
		UBTComposite_SimpleParallel* NodeOb = NewObject<UBTComposite_SimpleParallel>(&TreeOb);
		NodeOb->FinishMode = Mode;
		NodeOb->InitializeFromAsset(TreeOb);
		TreeOb.RootNode = NodeOb;
		return *NodeOb;
	}

	static UBTComposite_SimpleParallel& AddParallel(UBTCompositeNode& ParentNode, EBTParallelMode::Type Mode)
	{
		UBTComposite_SimpleParallel* NodeOb = NewObject<UBTComposite_SimpleParallel>(ParentNode.GetTreeAsset());
		NodeOb->FinishMode = Mode;
		NodeOb->InitializeFromAsset(*ParentNode.GetTreeAsset());

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildComposite = NodeOb;

		return *NodeOb;
	}

	template<class T>
	static T& AddTask(UBTCompositeNode& ParentNode, UClass* TaskClass = T::StaticClass())
	{
		T* TaskNode = NewObject<T>(ParentNode.GetTreeAsset());
		ParentNode.Children.Emplace_GetRef().ChildTask = TaskNode;

		return *TaskNode;
	}

	static void AddTask(UBTCompositeNode& ParentNode, int32 LogIndex, EBTNodeResult::Type NodeResult, int32 ExecutionTicks = 0, int32 LogTickIndex = -1)
	{
		UTestBTTask_Log* TaskNode = NewObject<UTestBTTask_Log>(ParentNode.GetTreeAsset());
		TaskNode->LogIndex = LogIndex;
		TaskNode->LogResult = NodeResult;
		TaskNode->ExecutionTicks = ExecutionTicks;
		TaskNode->LogTickIndex = LogTickIndex;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskLogFinish(UBTCompositeNode& ParentNode, int32 LogIndex, int32 FinishIndex, EBTNodeResult::Type NodeResult, int32 ExecutionTicks = 0)
	{
		UTestBTTask_Log* TaskNode = NewObject<UTestBTTask_Log>(ParentNode.GetTreeAsset());
		TaskNode->LogIndex = LogIndex;
		TaskNode->LogFinished = FinishIndex;
		TaskNode->LogResult = NodeResult;
		TaskNode->ExecutionTicks = ExecutionTicks;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskFlagChange(UBTCompositeNode& ParentNode, bool bValue, EBTNodeResult::Type NodeResult, FName BoolKeyName = TEXT("Bool1"), FName BoolOnAbortKeyName = FName(), bool bOnAbortValue = false)
	{
		UTestBTTask_SetFlag* TaskNode = NewObject<UTestBTTask_SetFlag>(ParentNode.GetTreeAsset());
		TaskNode->bValue = bValue;
		TaskNode->TaskResult = NodeResult;
		TaskNode->KeyName = BoolKeyName;
		TaskNode->OnAbortKeyName = BoolOnAbortKeyName;
		TaskNode->bOnAbortValue = bOnAbortValue;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskToggleFlag(UBTCompositeNode& ParentNode, EBTNodeResult::Type NodeResult, FName BoolKeyName, int32 NumToggles)
	{
		UTestBTTask_ToggleFlag* TaskNode = NewObject<UTestBTTask_ToggleFlag>(ParentNode.GetTreeAsset());
		TaskNode->TaskResult = NodeResult;
		TaskNode->KeyName = BoolKeyName;
		TaskNode->NumToggles = NumToggles;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskValueChange(UBTCompositeNode& ParentNode, int32 Value, EBTNodeResult::Type NodeResult, FName IntKeyName = TEXT("Int"), FName IntOnAbortKeyName = FName(), int32 OnAbortValue = 0)
	{
		UTestBTTask_SetValue* TaskNode = NewObject<UTestBTTask_SetValue>(ParentNode.GetTreeAsset());
		TaskNode->Value = Value;
		TaskNode->TaskResult = NodeResult;
		TaskNode->KeyName = IntKeyName;
		TaskNode->OnAbortKeyName = IntOnAbortKeyName;
		TaskNode->OnAbortValue = OnAbortValue;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskValuesChangedWithLogs(UBTCompositeNode& ParentNode, int32 LogIndex, EBTNodeResult::Type NodeResult, int32 Value1, int32 Value2, FName IntKeyName1 = TEXT("Int"), FName IntKeyName2 = TEXT("Int2"), int32 ExecutionTicks1 = 0, int32 ExecutionTicks2 = 0, int32 LogTickIndex = -1, int32 LogFinished = -1, FName IntOnAbortKeyName = FName(), int32 OnAbortValue = 0)
	{
		UTestBTTask_SetValuesWithLogs* TaskNode = NewObject<UTestBTTask_SetValuesWithLogs>(ParentNode.GetTreeAsset());
		TaskNode->LogIndex = LogIndex;
		TaskNode->LogFinished = LogFinished;
		TaskNode->ExecutionTicks1 = ExecutionTicks1;
		TaskNode->ExecutionTicks2 = ExecutionTicks2;
		TaskNode->LogTickIndex = LogTickIndex;
		TaskNode->KeyName1 = IntKeyName1;
		TaskNode->Value1 = Value1;
		TaskNode->KeyName2 = IntKeyName2;
		TaskNode->Value2 = Value2;
		TaskNode->OnAbortKeyName = IntOnAbortKeyName;
		TaskNode->OnAbortValue = OnAbortValue;
		TaskNode->TaskResult = NodeResult;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskSubtree(UBTCompositeNode& ParentNode, UBehaviorTree* TreeAsset)
	{
		UBTTask_RunBehavior* TaskNode = NewObject<UBTTask_RunBehavior>(ParentNode.GetTreeAsset());

		FObjectProperty* SubtreeProp = FindFProperty<FObjectProperty>(UBTTask_RunBehavior::StaticClass(), TEXT("BehaviorAsset"));
		uint8* SubtreePropData = SubtreeProp->ContainerPtrToValuePtr<uint8>(TaskNode);
		SubtreeProp->SetObjectPropertyValue(SubtreePropData, TreeAsset);

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskLatentFlags(UBTCompositeNode& ParentNode, EBTNodeResult::Type NodeResult,
		int32 ExecuteHalfTicks, /** Num of ticks before 'execute start' and `set execute flag` and then the same num of ticks before `execute finish` */
		FName ExecuteKeyName, int32 ExecuteLogStart, int32 ExecuteLogFinish,
		int32 AbortHalfTicks = 0, /** Num of ticks before 'abort start' and `set abort flag` and then the same num of ticks before `abort finish` */
		FName AbortKeyName = NAME_None, int32 AbortLogStart = 0, int32 AbortLogFinish = 0,
		EBTTestChangeFlagBehavior ChangeFlagBehavior = EBTTestChangeFlagBehavior::Set)
	{
		UTestBTTask_LatentWithFlags* TaskNode = NewObject<UTestBTTask_LatentWithFlags>(ParentNode.GetTreeAsset());
		TaskNode->ExecuteHalfTicks = ExecuteHalfTicks;
		TaskNode->KeyNameExecute = ExecuteKeyName;
		TaskNode->LogIndexExecuteStart = ExecuteLogStart;
		TaskNode->LogIndexExecuteFinish = ExecuteLogFinish;
		TaskNode->AbortHalfTicks = AbortHalfTicks;
		TaskNode->KeyNameAbort = AbortKeyName;
		TaskNode->LogIndexAbortStart = AbortLogStart;
		TaskNode->LogIndexAbortFinish = AbortLogFinish;
		TaskNode->ChangeFlagBehavior = ChangeFlagBehavior;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	static void AddTaskBTStopAction(UBTCompositeNode& ParentNode, int32 LogIndex, EBTNodeResult::Type NodeResult, EBTTestTaskStopTiming StopTiming, EBTTestStopAction StopAction)
	{
		UTestBTTask_BTStopAction* TaskNode = NewObject<UTestBTTask_BTStopAction>(ParentNode.GetTreeAsset());
		TaskNode->LogIndex = LogIndex;
		TaskNode->LogResult = NodeResult;
		TaskNode->StopTiming = StopTiming;
		TaskNode->StopAction = StopAction;

		const int32 ChildIdx = ParentNode.Children.AddZeroed(1);
		ParentNode.Children[ChildIdx].ChildTask = TaskNode;
	}

	template<class T>
	static T& WithDecorator(UBTCompositeNode& ParentNode, UClass* DecoratorClass = T::StaticClass())
	{
		T* DecoratorOb = NewObject<T>(ParentNode.GetTreeAsset());
		ParentNode.Children.Last().Decorators.Add(DecoratorOb);

		// Push additional memory validation decorator for extra coverage
		ParentNode.Children.Last().Decorators.Add(NewObject<UTestBTDecorator_NodeMemoryValidator>(ParentNode.GetTreeAsset()));

		return *DecoratorOb;
	}

	static void WithDecoratorBlackboard(UBTCompositeNode& ParentNode, EBasicKeyOperation::Type Condition,
		EBTFlowAbortMode::Type Observer, FName BoolKeyName = TEXT("Bool1"), 
		int32 LogIndexBecomeRelevant = -1, int32 LogIndexCeaseRelevant = -1, int32 LogIndexCalculate = -1)
	{
		UTestBTDecorator_Blackboard& BBDecorator = WithDecorator<UTestBTDecorator_Blackboard>(ParentNode);
		BBDecorator.LogIndexBecomeRelevant= LogIndexBecomeRelevant;
		BBDecorator.LogIndexCeaseRelevant= LogIndexCeaseRelevant;
		BBDecorator.LogIndexCalculate = LogIndexCalculate;

		FByteProperty* ConditionProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("OperationType"));
		uint8* ConditionPropData = ConditionProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		ConditionProp->SetIntPropertyValue(ConditionPropData, (uint64)Condition);

		FByteProperty* ObserverProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("FlowAbortMode"));
		uint8* ObserverPropData = ObserverProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		ObserverProp->SetIntPropertyValue(ObserverPropData, (uint64)Observer);

		FStructProperty* KeyProp = FindFProperty<FStructProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("BlackboardKey"));
		FBlackboardKeySelector* KeyPropData = KeyProp->ContainerPtrToValuePtr<FBlackboardKeySelector>(&BBDecorator);
		KeyPropData->SelectedKeyName = BoolKeyName;
	}

	static void WithDecoratorBlackboard(UBTCompositeNode& ParentNode, EArithmeticKeyOperation::Type Condition, int32 Value,
		EBTFlowAbortMode::Type Observer, EBTBlackboardRestart::Type NotifyMode, FName IntKeyName = TEXT("Int"),
		int32 LogIndexBecomeRelevant = -1, int32 LogIndexCeaseRelevant = -1, int32 LogIndexCalculate = -1)
	{
		UTestBTDecorator_Blackboard& BBDecorator = WithDecorator<UTestBTDecorator_Blackboard>(ParentNode);
		BBDecorator.LogIndexBecomeRelevant = LogIndexBecomeRelevant;
		BBDecorator.LogIndexCeaseRelevant = LogIndexCeaseRelevant;
		BBDecorator.LogIndexCalculate = LogIndexCalculate;

		FByteProperty* ConditionProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("OperationType"));
		uint8* ConditionPropData = ConditionProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		ConditionProp->SetIntPropertyValue(ConditionPropData, (uint64)Condition);

		FByteProperty* ObserverProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("FlowAbortMode"));
		uint8* ObserverPropData = ObserverProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		ObserverProp->SetIntPropertyValue(ObserverPropData, (uint64)Observer);

		FByteProperty* NotifyModeProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("NotifyObserver"));
		uint8* NotifyModePropData = NotifyModeProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		NotifyModeProp->SetIntPropertyValue(NotifyModePropData, (uint64)NotifyMode);

		FIntProperty* ConditionValueProp = FindFProperty<FIntProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("IntValue"));
		uint8* ConditionValuePropData = ConditionValueProp->ContainerPtrToValuePtr<uint8>(&BBDecorator);
		ConditionValueProp->SetIntPropertyValue(ConditionValuePropData, (uint64)Value);

		FStructProperty* KeyProp = FindFProperty<FStructProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("BlackboardKey"));
		FBlackboardKeySelector* KeyPropData = KeyProp->ContainerPtrToValuePtr<FBlackboardKeySelector>(&BBDecorator);
		KeyPropData->SelectedKeyName = IntKeyName;
	}

	static void WithDecoratorDelayedAbort(UBTCompositeNode& ParentNode, int32 NumTicks, bool bAbortOnlyOnce = true)
	{
		UTestBTDecorator_DelayedAbort& AbortDecorator = WithDecorator<UTestBTDecorator_DelayedAbort>(ParentNode);
		AbortDecorator.DelayTicks = NumTicks;
		AbortDecorator.bOnlyOnce = bAbortOnlyOnce;
	}

	static void WithDecoratorBlueprint(UBTCompositeNode& ParentNode, EBTFlowAbortMode::Type Observer, EBPConditionType BPConditionType = EBPConditionType::TrueCondition,
		int32 LogIndexBecomeRelevant = -1, int32 LogIndexCeaseRelevant = -1, int32 LogIndexCalculate = -1, FName ObservingKeyName = NAME_None )
	{
		UTestBTDecorator_Blueprint& BPDecorator = WithDecorator<UTestBTDecorator_Blueprint>(ParentNode);
		BPDecorator.LogIndexBecomeRelevant = LogIndexBecomeRelevant;
		BPDecorator.LogIndexCeaseRelevant = LogIndexCeaseRelevant;
		BPDecorator.LogIndexCalculate = LogIndexCalculate;
		BPDecorator.BPConditionType = BPConditionType;
		BPDecorator.ObservingKeyName = ObservingKeyName;

		FByteProperty* ObserverProp = FindFProperty<FByteProperty>(UBTDecorator_Blackboard::StaticClass(), TEXT("FlowAbortMode"));
		uint8* ObserverPropData = ObserverProp->ContainerPtrToValuePtr<uint8>(&BPDecorator);
		ObserverProp->SetIntPropertyValue(ObserverPropData, (uint64)Observer);
	}

	static void WithDecoratorLoop(UBTCompositeNode& ParentNode, int32 NumLoops = 2)
	{
		UBTDecorator_Loop& LoopDecorator = WithDecorator<UBTDecorator_Loop>(ParentNode);
		LoopDecorator.NumLoops = NumLoops;
	}

	static void WithDecoratorLoopUntil(UBTCompositeNode& ParentNode, EBTNodeResult::Type RequiredResult)
	{
		UTestBTDecorator_LoopUntil& LoopUntilDecorator = WithDecorator<UTestBTDecorator_LoopUntil>(ParentNode);
		LoopUntilDecorator.SetRequiredResult(RequiredResult);
	}
	
	template<class T>
	static T& WithService(UBTCompositeNode& ParentNode, UClass* ServiceClass = T::StaticClass())
	{
		T* ServiceOb = NewObject<T>(ParentNode.GetTreeAsset());
		ParentNode.Services.Add(ServiceOb);

		return *ServiceOb;
	}

	static void WithServiceLog(UBTCompositeNode& ParentNode, int32 ActivationIndex, int32 DeactivationIndex, int32 TickIndex = INDEX_NONE, FName TickBoolKeyName = NAME_None, bool bCallTickOnSearchStart = false, FName BecomeRelevantBoolKeyName = NAME_None, FName CeaseRelevantBoolKeyName = NAME_None, bool bToggleValue = false, int32 TicksDelaySetKeyNameTick = 0)
	{
		UTestBTService_Log& LogService = WithService<UTestBTService_Log>(ParentNode);
		LogService.LogActivation = ActivationIndex;
		LogService.LogDeactivation = DeactivationIndex;
		LogService.LogTick = TickIndex;
		LogService.SetFlagOnTick(TickBoolKeyName, bCallTickOnSearchStart);
		LogService.KeyNameBecomeRelevant = BecomeRelevantBoolKeyName;
		LogService.KeyNameCeaseRelevant = CeaseRelevantBoolKeyName;
		LogService.bToggleValue = bToggleValue;
		LogService.TicksDelaySetKeyNameTick = TicksDelaySetKeyNameTick;
	}

	static void WithServiceBTStopAction(UBTCompositeNode& ParentNode, int32 LogIndex, EBTTestServiceStopTiming StopTiming, EBTTestStopAction StopAction)
	{
		UTestBTService_BTStopAction& Service = WithService<UTestBTService_BTStopAction>(ParentNode);
		Service.LogIndex = LogIndex;
		Service.StopTiming = StopTiming;
		Service.StopAction = StopAction;
	}

	template<class T>
	static T& WithTaskService(UBTCompositeNode& ParentNode, UClass* ServiceClass = T::StaticClass())
	{
		UBTTaskNode* TaskNode = ParentNode.Children.Last().ChildTask;
		check(TaskNode);

		T* ServiceOb = NewObject<T>(ParentNode.GetTreeAsset());
		TaskNode->Services.Add(ServiceOb);

		return *ServiceOb;
	}

	static void WithTaskServiceLog(UBTCompositeNode& ParentNode, int32 ActivationIndex, int32 DeactivationIndex, int32 TickIndex = INDEX_NONE, FName TickBoolKeyName = NAME_None, bool bCallTickOnSearchStart = false, FName BecomeRelevantBoolKeyName = NAME_None, FName CeaseRelevantBoolKeyName = NAME_None, bool bToggleValue = false)
	{
		UTestBTService_Log& LogService = WithTaskService<UTestBTService_Log>(ParentNode);
		LogService.LogActivation = ActivationIndex;
		LogService.LogDeactivation = DeactivationIndex;
		LogService.LogTick = TickIndex;
		LogService.SetFlagOnTick(TickBoolKeyName, bCallTickOnSearchStart);
		LogService.KeyNameBecomeRelevant = BecomeRelevantBoolKeyName;
		LogService.KeyNameCeaseRelevant = CeaseRelevantBoolKeyName;
		LogService.bToggleValue = bToggleValue;
	}

	static void WithTaskServiceBTStopAction(UBTCompositeNode& ParentNode, int32 LogIndex, EBTTestServiceStopTiming StopTiming, EBTTestStopAction StopAction)
	{
		UTestBTService_BTStopAction& Service = WithTaskService<UTestBTService_BTStopAction>(ParentNode);
		Service.LogIndex = LogIndex;
		Service.StopTiming = StopTiming;
		Service.StopAction = StopAction;
	}

};
