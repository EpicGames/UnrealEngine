// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

template<typename ValueType>
struct FTestLogger
{
	TArray<ValueType> ExpectedValues;
	TArray<ValueType> LoggedValues;
	FAutomationTestBase* TestRunner;

	FTestLogger(FAutomationTestBase* InTestRunner = nullptr) : TestRunner(InTestRunner)
	{

	}

	~FTestLogger()
	{
		TestRunner->TestTrue("Not all values expected values have been logged!", ExpectedValues.Num() == 0 || LoggedValues.Num() == ExpectedValues.Num());
	}

	void Log(const ValueType& Value)
	{
		LoggedValues.Add(Value);

		if (ExpectedValues.Num() > 0 && TestRunner != nullptr)
		{
			if (LoggedValues.Num() <= ExpectedValues.Num())
			{
				TestRunner->TestEqual("Logged value different then expected!", LoggedValues.Top(), ExpectedValues[LoggedValues.Num() - 1]);
			}
			else
			{
				TestRunner->TestTrue("Logged more values than expected!", false);
			}
		}
	}
};
