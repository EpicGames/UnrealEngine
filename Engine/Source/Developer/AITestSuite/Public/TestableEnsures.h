// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "HAL/Platform.h"

AITESTSUITE_API DECLARE_LOG_CATEGORY_EXTERN(LogTestableEnsures, Log, All);

//-----------------------------------------------------------------------------
// support for testable ensure and check
//-----------------------------------------------------------------------------
namespace UE::AITestSuite
{
	AITESTSUITE_API extern int32 TestsInProgress;
}

#define AITEST_SCOPED_CHECK(PartialTextToExpect, NumOccurrences) \
	GetTestRunner().AddExpectedErrorPlain(PartialTextToExpect, EAutomationExpectedErrorFlags::Contains, NumOccurrences); \
	ON_SCOPE_EXIT \
	{ \
		--UE::AITestSuite::TestsInProgress; \
		FAutomationTestExecutionInfo ExecutionInfo; \
		GetTestRunner().GetExecutionInfo(ExecutionInfo); \
		if (ExecutionInfo.GetWarningTotal() + ExecutionInfo.GetErrorTotal() > 0) \
		{ \
			UE::AITestSuite::ConditionallyBreakOnTestFail(); \
		} \
	}; \
	++UE::AITestSuite::TestsInProgress

#define aiTestableEnsureMsgf(InExpression, InFormat, ... ) \
	(LIKELY(!!(InExpression)) || ([&]() \
		{ \
			if (UNLIKELY(UE::AITestSuite::TestsInProgress > 0)) \
			{ \
				UE_LOG(LogTestableEnsures, Warning, InFormat, ##__VA_ARGS__); \
			} \
			else \
			{ \
				ensureMsgf(InExpression, InFormat, ##__VA_ARGS__); \
			} \
		return false; \
		} ()))

#define aiTestableCheckf(InExpression, InFormat, ... ) \
	if (UNLIKELY(UE::AITestSuite::TestsInProgress > 0)) \
	{ \
		if (!(InExpression))\
		{ \
			UE_LOG(LogTestableEnsures, Error, InFormat, ##__VA_ARGS__); \
			return; \
		}\
	} \
	else \
	{ \
		checkf(InExpression, InFormat, ##__VA_ARGS__); \
	}

#define aiTestableCheckfReturn(InExpression, ReturnExpression, InFormat, ... ) \
	if (UNLIKELY(UE::AITestSuite::TestsInProgress > 0)) \
	{ \
		if (!(InExpression)) \
		{ \
			UE_LOG(LogTestableEnsures, Error, InFormat, ##__VA_ARGS__); \
			ReturnExpression; \
		} \
	} \
	else \
	{ \
		checkf(InExpression, InFormat, ##__VA_ARGS__); \
	}
